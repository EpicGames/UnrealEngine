{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PSD Importer",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "PSDImporterEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "PSDImporter",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "PSDImporterCore",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "GeometryMask",
			"Enabled": true
		}
	]
}
