// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDDocument.h"

#include "EditorFramework/AssetImportData.h"

UPSDDocument::UPSDDocument()
{
#if WITH_EDITORONLY_DATA
	AssetImportData = CreateEditorOnlyDefaultSubobject<UAssetImportData>(TEXT("AssetImportData"));
#endif
}

const FString& UPSDDocument::GetDocumentName() const
{
	return DocumentName;
}

const FIntPoint& UPSDDocument::GetSize() const
{
	return Size;
}

const TArray<FPSDFileLayer>& UPSDDocument::GetLayers() const
{
	return Layers;
}

bool UPSDDocument::WereLayersResizedOnImport() const
{
	return bLayersResizedOnImport;
}

TArray<const FPSDFileLayer*> UPSDDocument::GetValidLayers() const
{
	TArray<const FPSDFileLayer*> ValidLayers;

	for (const FPSDFileLayer& Layer : Layers)
	{
		if (!Layer.bIsSupportedLayerType || !Layer.bIsVisible)
		{
			continue;
		}

		if (FMath::IsNearlyZero(Layer.Opacity))
		{
			continue;
		}

		if (Layer.Bounds.Width() == 0 || Layer.Bounds.Height() == 0)
		{
			continue;
		}

		if (Layer.Texture.IsNull())
		{
			continue;
		}

		ValidLayers.Add(&Layer);
	}

	return ValidLayers;
}

int32 UPSDDocument::GetTextureCount() const
{
	int32 TextureCount = 0;

	for (const FPSDFileLayer* Layer : GetValidLayers())
	{
		++TextureCount;

		if (!Layer->Mask.IsNull())
		{
			++TextureCount;
		}
	}

	return TextureCount;
}

#if WITH_EDITOR
void UPSDDocument::GetAssetRegistryTags(FAssetRegistryTagsContext InContext) const
{
	UObject::GetAssetRegistryTags(InContext);

	if (IsValid(AssetImportData))
	{
		InContext.AddTag(FAssetRegistryTag(
			SourceFileTagName(), 
			AssetImportData->GetSourceData().ToJson(), 
			FAssetRegistryTag::TT_Hidden
		));
	}
}
#endif
