// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDQuadActor.h"

#include "Components/SceneComponent.h"
#include "PSDDocument.h"
#include "PSDQuadMeshActor.h"

APSDQuadActor::APSDQuadActor()
{
	LayerRoot = CreateDefaultSubobject<USceneComponent>("LayerRoot");
	SetRootComponent(LayerRoot);
}

UPSDDocument* APSDQuadActor::GetPSDDocument() const
{
	return PSDDocument;
}

TArray<APSDQuadMeshActor*> APSDQuadActor::GetQuadMeshes() const
{
	TArray<APSDQuadMeshActor*> MeshList;
	MeshList.Reserve(MeshListWeak.Num());

	for (const TWeakObjectPtr<APSDQuadMeshActor>& MeshWeak : MeshListWeak)
	{
		if (APSDQuadMeshActor* Mesh = MeshWeak.Get())
		{
			MeshList.Add(Mesh);
		}
	}

	return MeshList;
}

float APSDQuadActor::GetLayerDepthOffset() const
{
	return LayerDepthOffset;
}

void APSDQuadActor::SetLayerDepthOffset(float InDistance)
{
	if (FMath::IsNearlyEqual(LayerDepthOffset, InDistance))
	{
		return;
	}

	LayerDepthOffset = InDistance;

	OnLayerDepthOffsetChanged();
}

bool APSDQuadActor::IsAdjustingForViewDistance() const
{
	return AdjustForViewDistance > 0.0 && !FMath::IsNearlyZero(AdjustForViewDistance);
}

float APSDQuadActor::GetAdjustForViewDistance() const
{
	return AdjustForViewDistance;
}

void APSDQuadActor::SetAdjustForViewDistance(float InDistance)
{
	if (FMath::IsNearlyEqual(AdjustForViewDistance, InDistance))
	{
		return;
	}

	AdjustForViewDistance = InDistance;

	OnAdjustForViewDistanceChanged();
}

bool APSDQuadActor::IsSettingTranslucentSortPriority() const
{
	return BaseTranslucentSortPriority != 0;
}

void APSDQuadActor::SetBaseTranslucentSortPriority(int32 InPriority)
{
	if (BaseTranslucentSortPriority == InPriority)
	{
		return;
	}

	BaseTranslucentSortPriority = InPriority;

	OnBaseTranslucentSortPriorityChanged();
}

float APSDQuadActor::GetBaseTranslucentSortPriority() const
{
	return BaseTranslucentSortPriority;
}

#if WITH_EDITOR
void APSDQuadActor::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, LayerDepthOffset))
	{
		OnLayerDepthOffsetChanged();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, AdjustForViewDistance))
	{
		OnAdjustForViewDistanceChanged();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, BaseTranslucentSortPriority))
	{
		OnBaseTranslucentSortPriorityChanged();
	}
}

void APSDQuadActor::SetPSDDocument(UPSDDocument& InPSDDocument)
{
	PSDDocument = &InPSDDocument;

	SetActorLabel(InPSDDocument.GetDocumentName());

	MeshListWeak.Reset(InPSDDocument.GetLayers().Num());
}

void APSDQuadActor::AddQuadMesh(APSDQuadMeshActor& InMeshActor)
{
	MeshListWeak.Add(&InMeshActor);
}

void APSDQuadActor::InitComplete()
{
	UpdateQuadSeparationDistances();
}

FString APSDQuadActor::GetDefaultActorLabel() const
{
	return TEXT("PSD Layer Root Actor");
}
#endif

void APSDQuadActor::OnLayerDepthOffsetChanged()
{
	UpdateQuadSeparationDistances();
	UpdateQuadSizeForViewDistance();
}

void APSDQuadActor::UpdateQuadSeparationDistances()
{
	for (APSDQuadMeshActor* MeshActor : GetQuadMeshes())
	{
		MeshActor->ResetQuadDepth();
	}
}

void APSDQuadActor::OnAdjustForViewDistanceChanged()
{
	UpdateQuadSeparationDistances();
	UpdateQuadSizeForViewDistance();
}

void APSDQuadActor::UpdateQuadSizeForViewDistance()
{
	for (APSDQuadMeshActor* MeshActor : GetQuadMeshes())
	{
		MeshActor->ResetQuadSize();
		MeshActor->ResetQuadPosition();
	}
}

void APSDQuadActor::OnBaseTranslucentSortPriorityChanged()
{
	UpdateQuadTranslucency();
}

void APSDQuadActor::UpdateQuadTranslucency()
{
	for (APSDQuadMeshActor* MeshActor : GetQuadMeshes())
	{
		MeshActor->ResetQuadTranslucentSortPriority();
	}
}

void APSDQuadActor::Destroyed()
{
	Super::Destroyed();

	for (APSDQuadMeshActor* MeshActor : GetQuadMeshes())
	{
		MeshActor->Destroy();
	}
}
