// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDQuadMeshActor.h"

#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "PSDDocument.h"
#include "PSDQuadActor.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#endif

namespace UE::PSDImporter::Private
{
	constexpr const TCHAR* PlaneMeshPath = TEXT("/Script/Engine.StaticMesh'/PSDImporter/PSDImporter/QuadMesh.QuadMesh'");

	UStaticMesh* GetQuadMesh()
	{
		const TSoftObjectPtr<UStaticMesh> StaticMeshPtr = TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(PlaneMeshPath));
		return StaticMeshPtr.LoadSynchronous();
	}

#if WITH_EDITOR
	void RequestViewportRedraw()
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::Get().GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));

		if (!LevelEditorModule)
		{
			return;
		}

		LevelEditorModule->BroadcastRedrawViewports(/* Invalidate hit proxies */ true);
	}
#endif
}

FPSDImporterTextureResetDelegate APSDQuadMeshActor::TextureResetDelegate;

APSDQuadMeshActor::APSDQuadMeshActor()
{
	using namespace UE::PSDImporter::Private;

	Mesh = CreateDefaultSubobject<UStaticMeshComponent>("Mesh");
	Mesh->SetStaticMesh(GetQuadMesh());
	SetRootComponent(Mesh);	
}

APSDQuadActor* APSDQuadMeshActor::GetQuadActor() const
{
	return QuadActorWeak.Get();
}

const FPSDFileLayer* APSDQuadMeshActor::GetLayer() const
{
	APSDQuadActor* QuadActor = GetQuadActor();

	if (!QuadActor)
	{
		return nullptr;
	}

	UPSDDocument* PSDDocument = QuadActor->GetPSDDocument();

	if (!PSDDocument || !PSDDocument->GetLayers().IsValidIndex(LayerIndex))
	{
		return nullptr;
	}

	return &PSDDocument->GetLayers()[LayerIndex];
}

const FPSDFileLayer* APSDQuadMeshActor::GetClippingLayer() const
{
	APSDQuadActor* QuadActor = GetQuadActor();

	if (!QuadActor)
	{
		return nullptr;
	}

	const FPSDFileLayer* Layer = GetLayer();

	if (!Layer || Layer->Clipping == 0)
	{
		return nullptr;
	}

	UPSDDocument* PSDDocument = QuadActor->GetPSDDocument();

	if (!PSDDocument || !PSDDocument->GetLayers().IsValidIndex(LayerIndex - 1))
	{
		return nullptr;
	}

	return &PSDDocument->GetLayers()[LayerIndex - 1];
}

UMaterialInterface* APSDQuadMeshActor::GetQuadMaterial() const
{
	if (Mesh)
	{
		return Mesh->GetMaterial(0);
	}

	return nullptr;
}

void APSDQuadMeshActor::ResetQuad()
{
	ResetQuadDepth();
	ResetQuadPosition();
	ResetQuadSize();
	ResetQuadTexture();
	ResetQuadTranslucentSortPriority();
}

void APSDQuadMeshActor::ResetQuadDepth()
{
	if (LayerIndex < 0)
	{
		return;
	}

	APSDQuadActor* QuadActor = GetQuadActor();

	if (!QuadActor)
	{
		return;
	}

	if (!Mesh)
	{
		return;
	}

	FVector Offset = Mesh->GetRelativeLocation();
	Offset.X = QuadActor->GetLayerDepthOffset() * static_cast<float>(LayerIndex) * -1.0;

	Mesh->SetRelativeLocation(Offset);
	Mesh->MarkRenderTransformDirty();

#if WITH_EDITOR
	UE::PSDImporter::Private::RequestViewportRedraw();
#endif
}

void APSDQuadMeshActor::ResetQuadPosition()
{
	if (!Mesh)
	{
		return;
	}

	APSDQuadActor* QuadActor = GetQuadActor();

	if (!QuadActor)
	{
		return;
	}

	UPSDDocument* Document = QuadActor->GetPSDDocument();

	if (!Document)
	{
		return;
	}

	const FPSDFileLayer* Layer = GetClippingLayer();

	if (!Layer)
	{
		Layer = GetLayer();

		if (!Layer)
		{
			return;
		}
	}

	const FIntPoint DocumentSize = Document->GetSize();
	const FIntPoint LayerPosition = Document->WereLayersResizedOnImport()
		? FIntPoint::ZeroValue
		: Layer->Bounds.Min;
	const FIntPoint LayerSize = Document->WereLayersResizedOnImport()
		? DocumentSize
		: FIntPoint(Layer->Bounds.Width(), Layer->Bounds.Height());

	FVector Offset = Mesh->GetRelativeLocation();
	Offset.Y = (-DocumentSize.X / 2) + (LayerSize.X / 2) + LayerPosition.X;
	Offset.Z = (DocumentSize.Y / 2) - (LayerSize.Y / 2) - LayerPosition.Y;

	if (QuadActor->IsAdjustingForViewDistance())
	{
		const double ViewAdjustRatio = (QuadActor->GetAdjustForViewDistance() - (LayerIndex * QuadActor->GetLayerDepthOffset())) / QuadActor->GetAdjustForViewDistance();
		Offset.X *= ViewAdjustRatio;
		Offset.Y *= ViewAdjustRatio;
	}

	Mesh->SetRelativeLocation(Offset);
	Mesh->MarkRenderTransformDirty();

#if WITH_EDITOR
	UE::PSDImporter::Private::RequestViewportRedraw();
#endif
}

void APSDQuadMeshActor::ResetQuadSize()
{
	if (!Mesh)
	{
		return;
	}

	APSDQuadActor* QuadActor = GetQuadActor();

	if (!QuadActor)
	{
		return;
	}

	UPSDDocument* Document = QuadActor->GetPSDDocument();

	if (!Document)
	{
		return;
	}

	const FPSDFileLayer* Layer = GetClippingLayer();

	if (!Layer)
	{
		Layer = GetLayer();

		if (!Layer)
		{
			return;
		}
	}

	UStaticMesh* QuadStaticMesh = Mesh->GetStaticMesh();

	if (!QuadStaticMesh)
	{
		return;
	}

	const FIntPoint DocumentSize = Document->GetSize();
	const FIntPoint LayerSize = Document->WereLayersResizedOnImport()
		? DocumentSize
		: FIntPoint(Layer->Bounds.Width(), Layer->Bounds.Height());

	const FVector MeshBounds = QuadStaticMesh->GetBoundingBox().GetSize();

	FVector Scale = Mesh->GetRelativeScale3D();
	Scale.Y = static_cast<double>(LayerSize.X) / MeshBounds.Y;
	Scale.Z = static_cast<double>(LayerSize.Y) / MeshBounds.Z;

	if (QuadActor->IsAdjustingForViewDistance())
	{
		const double ViewAdjustRatio = (QuadActor->GetAdjustForViewDistance() - (LayerIndex * QuadActor->GetLayerDepthOffset())) / QuadActor->GetAdjustForViewDistance();
		Scale.Y *= ViewAdjustRatio;
		Scale.Z *= ViewAdjustRatio;
	}

	Mesh->SetRelativeScale3D(Scale);
	Mesh->MarkRenderTransformDirty();

#if WITH_EDITOR
	UE::PSDImporter::Private::RequestViewportRedraw();
#endif
}

void APSDQuadMeshActor::ResetQuadTexture()
{
	if (!Mesh)
	{
		return;
	}

	UMaterialInterface* LayerMaterial = Cast<UMaterialInterface>(Mesh->GetMaterial(0));

	if (!LayerMaterial)
	{
		return;
	}

	const FPSDFileLayer* Layer = GetLayer();

	if (!Layer)
	{
		return;
	}

	TextureResetDelegate.Broadcast(*this);

	Mesh->MarkRenderStateDirty();

#if WITH_EDITOR
	UE::PSDImporter::Private::RequestViewportRedraw();
#endif
}

void APSDQuadMeshActor::ResetQuadTranslucentSortPriority()
{
	if (!Mesh)
	{
		return;
	}

	APSDQuadActor* QuadActor = GetQuadActor();

	if (!QuadActor)
	{
		return;
	}

	const int32 BaseTranslucentSortPriority = QuadActor->GetBaseTranslucentSortPriority();

	if (BaseTranslucentSortPriority == 0)
	{
		Mesh->SetTranslucentSortPriority(0);
	}
	else
	{
		Mesh->SetTranslucentSortPriority(LayerIndex + QuadActor->GetBaseTranslucentSortPriority());
	}

	Mesh->MarkRenderStateDirty();

#if WITH_EDITOR
	UE::PSDImporter::Private::RequestViewportRedraw();
#endif
}

#if WITH_EDITOR
FString APSDQuadMeshActor::GetDefaultActorLabel() const
{
	return TEXT("Layer Actor");
}

void APSDQuadMeshActor::InitLayer(APSDQuadActor& InQuadActor, int32 InLayerIndex, UMaterialInterface* InLayerMaterial)
{
	QuadActorWeak = &InQuadActor;
	LayerIndex = InLayerIndex;

	AttachToActor(&InQuadActor, FAttachmentTransformRules::SnapToTargetNotIncludingScale);

	if (Mesh)
	{
		Mesh->SetMaterial(0, InLayerMaterial);
	}

	if (const FPSDFileLayer* Layer = GetLayer())
	{
		SetActorLabel(FString::Printf(TEXT("[%03d] %s"), InLayerIndex, *Layer->Id.Name));
	}

	ResetQuad();
}
#endif
