// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "PSDFile.h"

#include "PSDDocument.generated.h"

class UAssetImportData;
class UPSDLayerStack;

/**
 * @class UPSDDocument
 * @brief Represents a PSD document in Unreal Engine.
 *
 * This class is used to store information about a PSD document, such as its name, size, and layers.
 * It is intended to be used as part of the PSD Importer API in Unreal Engine.
 */
UCLASS(MinimalAPI, BlueprintType)
class UPSDDocument
	: public UObject
{
	GENERATED_BODY()

	friend class FPSDFileImporter;
	friend class UPSDDocumentImportFactory;
	friend struct FPSDDocumentImportFactory_Visitors;

public:
	PSDIMPORTER_API UPSDDocument();

	UFUNCTION(BlueprintPure, Category = "PSD Document")
	PSDIMPORTER_API const FString& GetDocumentName() const;

	UFUNCTION(BlueprintPure, Category = "PSD Document")
	PSDIMPORTER_API const FIntPoint& GetSize() const;

	UFUNCTION(BlueprintPure, Category = "PSD Document")
	PSDIMPORTER_API const TArray<FPSDFileLayer>& GetLayers() const;

	UFUNCTION(BlueprintPure, Category = "PSD Document")
	PSDIMPORTER_API bool WereLayersResizedOnImport() const;

	/** Returns the layers with a valid size, that are visible, that aren't completely opaque and have a supported type. */
	PSDIMPORTER_API TArray<const FPSDFileLayer*> GetValidLayers() const;

	/** @return the number of textures, including mask textures, of "valid" layers. */
	PSDIMPORTER_API int32 GetTextureCount() const;

#if WITH_EDITOR
	//~ Begin UObject
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext InContext) const override;
	//~ End UObject
#endif

private:
	/** Original document name, the asset may differ (if the user renamed it). */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "PSD Document", meta = (AllowPrivateAccess = "true"))
	FString DocumentName;
	
	/** Resolution (in pixels); */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "PSD Document", meta = (AllowPrivateAccess = "true"))
	FIntPoint Size;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, NoClear, EditFixedSize, Category = "PSD Document", 
		meta = (AllowPrivateAccess = "true", ShowOnlyInnerProperties, EditFixedOrder, NoResetToDefault))
	TArray<FPSDFileLayer> Layers;

	UPROPERTY()
	bool bLayersResizedOnImport = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PSD Document|Import Settings", meta = (AllowPrivateAccess = "true"))
	bool bImportInvisibleLayers = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PSD Document|Import Settings", meta = (AllowPrivateAccess = "true"))
	bool bResizeLayersToDocument = false;

	UPROPERTY()
	FPSDFileDocument FileDocument;

	UPROPERTY(EditAnywhere, Instanced, Category = "PSD Document|Import Settings")
	TObjectPtr<UAssetImportData> AssetImportData;
#endif
};
