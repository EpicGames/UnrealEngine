// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "Engine/Texture2D.h"
#include "PSDFileData.h"
#include "UObject/Object.h"

#include "PSDFile.generated.h"

UENUM(BlueprintType)
enum class EPSDFileLayerImportOperation : uint8
{
	Ignore = 0,
	Import = 1,
	ImportMerged = 2,
	Rasterize = 4,
};
ENUM_CLASS_FLAGS(EPSDFileLayerImportOperation);

UENUM()
enum class EPSDFileLayerType : uint8
{
	Any = 0,
	Group = 1,
};

USTRUCT(BlueprintType)
struct FPSDFileLayerId
{
	GENERATED_BODY()

	/** Maps to the layer index in the PSD file. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	int32 Index = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	FString Name;

	FPSDFileLayerId()
		: FPSDFileLayerId(INDEX_NONE, FString(TEXT("")))
	{		
	}

	FPSDFileLayerId(int32 InIndex, const FString& InName)
		: Index(InIndex)
		, Name(InName)
	{		
	}

	bool operator==(const FPSDFileLayerId& InOther) const
	{
		return Index == InOther.Index
			&& Name == InOther.Name;
	}

	bool operator!=(const FPSDFileLayerId& InOther) const
	{
		return !(*this == InOther);
	}

	bool operator<(const FPSDFileLayerId& InOther) const
	{
		return Index < InOther.Index;
	}

	bool operator<=(const FPSDFileLayerId& InOther) const
	{
		return operator<(InOther) || operator==(InOther);
	}
};

inline uint32 GetTypeHash(const FPSDFileLayerId& InValue)
{
	return HashCombineFast(GetTypeHash(InValue.Index), GetTypeHash(InValue.Name));
}

USTRUCT(BlueprintType)
struct FPSDFileLayer
{
	GENERATED_BODY()

public:
	FPSDFileLayer() = default;

	/** Implicit conversion from Id */
	FPSDFileLayer(const FPSDFileLayerId& InId)
		: Id(InId)
		, Type(EPSDFileLayerType::Any)
		, Bounds()
		, BlendMode(EPSDBlendMode::PassThrough)
	{
	}

	explicit FPSDFileLayer(int32 InIndex, const FString& InName, EPSDFileLayerType InType)
		: Id(InIndex, InName)
		, Type(InType)
		, Bounds()
		, BlendMode(EPSDBlendMode::PassThrough)
	{
	}

	UPROPERTY(BlueprintReadOnly, Category = "Layer")
	TOptional<FPSDFileLayerId> ParentId;

	/** Index, Name tuple. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	FPSDFileLayerId Id;

	UPROPERTY(BlueprintReadOnly, Category = "Layer")
	EPSDFileLayerType Type = EPSDFileLayerType::Any;

	/** Bounds of the layer. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	FIntRect Bounds;

	/** Visibility state of the layer. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	bool bIsVisible = true;

	/** The blending mode applied to the layer. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	EPSDBlendMode BlendMode = EPSDBlendMode::PassThrough;

	/** Opacity level of the layer, value ranging from 0.0 (fully transparent) to 1.0 (fully opaque). */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer", meta = (ClampMin = 0.0, ClampMax = 1.0, UIMin = 0.0, UIMax = 1.0))
	double Opacity = 1.0;

	/** A flag indicating whether the layer's type is supported. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	bool bIsSupportedLayerType = true;

	/** User specified import operation/option. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Layer")
	EPSDFileLayerImportOperation ImportOperation = EPSDFileLayerImportOperation::Import;

	/** Optional thumbnail preview of the layer's contents. */
	UPROPERTY(VisibleAnywhere, NoClear, Category = "Layer")
	TObjectPtr<UTexture2D> ThumbnailTexture;

	/** Imported texture. */
	UPROPERTY(VisibleAnywhere, NoClear, Category = "Layer")
	TSoftObjectPtr<UTexture2D> Texture;

	/** Imported mask. */
	UPROPERTY(VisibleAnywhere, NoClear, Category = "Layer")
	TSoftObjectPtr<UTexture2D> Mask;

	/** Bounds of the mask. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	FIntRect MaskBounds;

	/** Value of the mask outside the bounds. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	float MaskDefaultValue = 1.f;

	/** Whether this is a clipping layer. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layer")
	uint8 Clipping = 0;

	bool operator==(const FPSDFileLayer& InOther) const
	{
		return Id == InOther.Id;
	}

	bool operator!=(const FPSDFileLayer& InOther) const
	{
		return !(*this == InOther);
	}

	bool operator<(const FPSDFileLayer& InOther) const
	{
		return Id < InOther.Id;
	}

	bool operator<=(const FPSDFileLayer& InOther) const
	{
		return operator<(InOther) || operator==(InOther);
	}

	bool HasMask() const
	{
		return !Mask.IsNull();
	}

	bool IsLayerFullSize(const FIntPoint& InDocumentSize) const
	{
		return Bounds.Min.X == 0 && Bounds.Min.Y == 0 && Bounds.Max.X == InDocumentSize.X && Bounds.Max.Y == InDocumentSize.Y;
	}

	bool IsMaskFullSize(const FIntPoint& InDocumentSize) const
	{
		return MaskBounds.Min.X == 0 && MaskBounds.Min.Y == 0 && MaskBounds.Max.X == InDocumentSize.X && MaskBounds.Max.Y == InDocumentSize.Y;
	}

	bool NeedsCrop(const FIntPoint& InDocumentSize) const
	{
		return !IsLayerFullSize(InDocumentSize) || (HasMask() && !IsMaskFullSize(InDocumentSize));
	}
};

inline uint32 GetTypeHash(const FPSDFileLayer& InValue)
{
	return GetTypeHash(InValue.Id);
}

/** Representation of a PSD file document. */
USTRUCT(BlueprintType)
struct FPSDFileDocument
{
	GENERATED_BODY()

	/** Height of the document in pixels. */
	UPROPERTY()
	int32 Height = 0;

	/** Width of the document in pixels. */
	UPROPERTY()
	int32 Width = 0;

	/** Bit depth of the document. It can be 8, 16 or 32. */
	UPROPERTY()
	uint8 Depth = 8;

	/** Color mode of the document e.g., RGB, CMYK, etc. */
	UPROPERTY()
	FName ColorMode = TEXT("RGB");

	/** Set of layers contained in the document. */
	UPROPERTY()
	TSet<FPSDFileLayer> Layers;
};
