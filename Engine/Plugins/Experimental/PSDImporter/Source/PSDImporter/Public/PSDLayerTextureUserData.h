// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"

#include "Math/Box2D.h"
#include "Math/IntRect.h"
#include "PSDFile.h"
#include "UObject/Object.h"

#include "PSDLayerTextureUserData.generated.h"

UCLASS(MinimalAPI)
class UPSDLayerTextureUserData
	: public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Layer")
	FPSDFileLayerId LayerId;

	/** Stored in 0.0-1.0 space. */
	UPROPERTY(VisibleAnywhere, Category = "Layer")
	FBox2D NormalizedBounds;

	/** Stored in pixel space. */
	UPROPERTY(VisibleAnywhere, Category = "Layer")
	FIntRect PixelBounds;
};
