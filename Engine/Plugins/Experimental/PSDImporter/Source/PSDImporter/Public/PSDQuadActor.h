// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"

#include "PSDQuadActor.generated.h"

class UPSDDocument;
class USceneComponent;
class APSDQuadMeshActor;

UCLASS(MinimalAPI, DisplayName = "PSD Layer Root Actor")
class APSDQuadActor : public AActor
{
	GENERATED_BODY()

public:
	APSDQuadActor();

	virtual ~APSDQuadActor() override = default;

	PSDIMPORTER_API UPSDDocument* GetPSDDocument() const;

	PSDIMPORTER_API TArray<APSDQuadMeshActor*> GetQuadMeshes() const;

	PSDIMPORTER_API float GetLayerDepthOffset() const;

	PSDIMPORTER_API void SetLayerDepthOffset(float InDistance);

	PSDIMPORTER_API bool IsAdjustingForViewDistance() const;

	PSDIMPORTER_API float GetAdjustForViewDistance() const;

	PSDIMPORTER_API void SetAdjustForViewDistance(float InDistance);

	PSDIMPORTER_API bool IsSettingTranslucentSortPriority() const;

	PSDIMPORTER_API float GetBaseTranslucentSortPriority() const;

	PSDIMPORTER_API void SetBaseTranslucentSortPriority(int32 InPriority);

#if WITH_EDITOR
	PSDIMPORTER_API void SetPSDDocument(UPSDDocument& InPSDDocument);

	PSDIMPORTER_API void AddQuadMesh(APSDQuadMeshActor& InMeshActor);

	PSDIMPORTER_API void InitComplete();

	// Begin AActor
	virtual FString GetDefaultActorLabel() const override;
	// End AActor

	//~ Begin UObject
	PSDIMPORTER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject
#endif

	//~ Begin UObject
	virtual void Destroyed() override;
	//~ End UObject

private:
	UPROPERTY()
	TObjectPtr<UPSDDocument> PSDDocument;

	UPROPERTY()
	TArray<TWeakObjectPtr<APSDQuadMeshActor>> MeshListWeak;

	UPROPERTY(VisibleAnywhere, Category = "PSD")
	TObjectPtr<USceneComponent> LayerRoot;

	/** WIll separate each layer by this amount. */
	UPROPERTY(EditInstanceOnly, Setter, Category = "PSD")
	float LayerDepthOffset = 1.f;

	/** Will reduce the size of nearer quads to account for view distance with respect to the layer separation distance. 0 to disable. */
	UPROPERTY()
	float AdjustForViewDistance = 0.f;

	/** When assigning sort priority, use this as the first layer's priority. 0 to disable setting sort priority. */
	UPROPERTY(EditInstanceOnly, Category = "PSD")
	int32 BaseTranslucentSortPriority = 1;

	void OnLayerDepthOffsetChanged();

	void UpdateQuadSeparationDistances();

	void OnAdjustForViewDistanceChanged();

	void UpdateQuadSizeForViewDistance();

	void OnBaseTranslucentSortPriorityChanged();

	void UpdateQuadTranslucency();
};
