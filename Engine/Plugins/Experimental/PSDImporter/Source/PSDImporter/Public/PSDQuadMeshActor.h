// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "UObject/WeakObjectPtr.h"

#include "PSDQuadMeshActor.generated.h"

class APSDQuadActor;
class APSDQuadMeshActor;
class UMaterialInterface;
class UStaticMeshActor;
class UTexture;
struct FPSDFileLayer;

namespace UE::PSDImporter
{
	constexpr const TCHAR* LayerTextureParameterName = TEXT("LayerTexture");
	constexpr const TCHAR* LayerBoundsParameterName = TEXT("LayerBounds");
	constexpr const TCHAR* MaskTextureParameterName = TEXT("MaskTexture");
	constexpr const TCHAR* MaskBoundsParameterName = TEXT("MaskBounds");
	constexpr const TCHAR* MaskDefaultValueParameterName = TEXT("MaskDefaultValue");
	constexpr const TCHAR* ClippingLayerTextureParameterName = TEXT("ClippingLayerTexture");
	constexpr const TCHAR* ClippingLayerBoundsParameterName = TEXT("ClippingLayerBounds");
	constexpr const TCHAR* ClippingMaskTextureParameterName = TEXT("ClippingMaskTexture");
	constexpr const TCHAR* ClippingMaskBoundsParameterName = TEXT("ClippingMaskBounds");
	constexpr const TCHAR* ClippingMaskDefaultValueParameterName = TEXT("ClippingMaskDefaultValue");
	constexpr int32 MaxSamplerCount = 32;
}

enum class EPSDImporterLayerMaterialType : uint8
{
	Default = 0,
	HasMask = 1 << 0,
	IsClipping = 1 << 1,
	ClipHasMask = 1 << 2
};
ENUM_CLASS_FLAGS(EPSDImporterLayerMaterialType)

/**
 * We cannot know exactly what sort of material is being used on the actor, so we create a way for other material
 * systems to hook into the reset call.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FPSDImporterTextureResetDelegate, APSDQuadMeshActor& /* Actor on which to reset the reset */)

UCLASS(MinimalAPI, DisplayName = "PSD Layer Actor")
class APSDQuadMeshActor : public AActor
{
	GENERATED_BODY()

	friend class UPSDQuadsFactory;

public:
	static FPSDImporterTextureResetDelegate::RegistrationType& GetTextureResetDelegate() { return TextureResetDelegate; }

	APSDQuadMeshActor();

	virtual ~APSDQuadMeshActor() override = default;

	PSDIMPORTER_API APSDQuadActor* GetQuadActor() const;

	PSDIMPORTER_API const FPSDFileLayer* GetLayer() const;

	PSDIMPORTER_API const FPSDFileLayer* GetClippingLayer() const;

	PSDIMPORTER_API UMaterialInterface* GetQuadMaterial() const;

#if WITH_EDITOR
	PSDIMPORTER_API void InitLayer(APSDQuadActor& InQuadActor, int32 InLayerIndex, UMaterialInterface* InLayerMaterial);
#endif

	UFUNCTION(CallInEditor, Category = "PSD", DisplayName = "Reset All")
	PSDIMPORTER_API void ResetQuad();

	UFUNCTION(CallInEditor, Category = "PSD", DisplayName = "Reset Depth")
	PSDIMPORTER_API void ResetQuadDepth();

	UFUNCTION(CallInEditor, Category = "PSD", DisplayName = "Reset Position")
	PSDIMPORTER_API void ResetQuadPosition();

	UFUNCTION(CallInEditor, Category = "PSD", DisplayName = "Reset Size")
	PSDIMPORTER_API void ResetQuadSize();

	UFUNCTION(CallInEditor, Category = "PSD", DisplayName = "Reset Texture")
	PSDIMPORTER_API void ResetQuadTexture();

	UFUNCTION(CallInEditor, Category = "PSD", DisplayName = "Reset Translucent Sort Priority")
	PSDIMPORTER_API void ResetQuadTranslucentSortPriority();

#if WITH_EDITOR
	// Begin AActor
	virtual FString GetDefaultActorLabel() const override;
	// End AActor
#endif

private:
	PSDIMPORTER_API static FPSDImporterTextureResetDelegate TextureResetDelegate;

	UPROPERTY(VisibleAnywhere, DisplayName = "Layer Root Actor", Category = "PSD")
	TWeakObjectPtr<APSDQuadActor> QuadActorWeak;

	UPROPERTY(VisibleAnywhere, Category = "PSD")
	int32 LayerIndex = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = "PSD")
	TObjectPtr<UStaticMeshComponent> Mesh;
};