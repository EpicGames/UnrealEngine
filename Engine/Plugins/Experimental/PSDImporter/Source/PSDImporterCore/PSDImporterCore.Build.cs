// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class PSDImporterCore : ModuleRules
{
	public PSDImporterCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "..", "ThirdParty", "PsdSDK", "Includes"));
		PublicSystemLibraryPaths.Add(Path.Combine(ModuleDirectory, "..", "ThirdParty", "PsdSDK", "Libraries"));

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"ImageCore",
				"ImageWrapper",
				"PsdSDK",
			}
		);
	}
}
