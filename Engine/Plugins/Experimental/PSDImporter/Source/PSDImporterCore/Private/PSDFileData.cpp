// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDFileData.h"

namespace UE::PSDImporter::File
{
	bool FPSDHeader::IsValid() const
	{
		return (Signature == 0x38425053) // Fail on bad signature
			&& (Version == 1 || Version == 2); // Fail on bad version
	}	

	const TCHAR* LexToString(const EPSDColorMode InEnum)
	{
		static TMap<EPSDColorMode, const TCHAR*> Lookup = {
			{ EPSDColorMode::Bitmap, TEXT("Bitmap") },
			{ EPSDColorMode::Grayscale, TEXT("Grayscale") },
			{ EPSDColorMode::Indexed, TEXT("Indexed") },
			{ EPSDColorMode::RGB, TEXT("RGB") },
			{ EPSDColorMode::CMYK, TEXT("CMYK") },
			{ EPSDColorMode::Multichannel, TEXT("Multichannel") },
			{ EPSDColorMode::Duotone, TEXT("Duotone") },
			{ EPSDColorMode::Lab, TEXT("Lab") },
		};

		if (const TCHAR** FoundStr = Lookup.Find(InEnum))
		{
			return *FoundStr;
		}

		return TEXT("Invalid");
	}
}
