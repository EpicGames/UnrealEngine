// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDFileImport.h"

#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "PSDFileDocument.h"
#include "PSDImporterCoreLog.h"
#include "Psd.h"
#include "PsdAllocator.h"
#include "PsdColorMode.h"
#include "PsdDocument.h"
#include "PsdFile.h"
#include "PsdNativeFile.h"
#include "PsdParseDocument.h"
#include "Readers/DocumentReader.h"
#include "Readers/LayerReader.h"
#include "Templates/SharedPointer.h"

namespace UE::PSDImporter::Private
{
	class FPSDFileImporterImpl
		: public FPSDFileImporter
	{
	public:
		FPSDFileImporterImpl(const FString& InFileName)
			: FPSDFileImporter()
			, FileName(InFileName)
		{
		}

		virtual ~FPSDFileImporterImpl() override
		{
			Close();
		}

		bool Open()
		{
			if (IsOpen())
			{
				return true;
			}
			
			Close();

			File = MakeShared<psd::NativeFile>(&Allocator);
			if (!File->OpenRead(StringCast<wchar_t>(*FileName).Get()))
			{
				Close();
				return false;
			}

			Document = psd::CreateDocument(File.Get(), &Allocator);
			if (!Document)
			{
				UE_LOGF(LogPSDImporterCore, Error, "Cannot open the document for file '%ls'", *FileName);
				Close();
				return false;
			}

			if (Document->colorMode != psd::colorMode::RGB)
			{
				UE_LOGF(LogPSDImporterCore, Warning, "PSD import only supports RGB as a color mode.")
				Close();
				return false;
			}

			OutputDocument = MakeShared<File::FPSDDocument>();

			// Successfully read, and supported
			bIsOpen = true;
			return true;
		}

		void Close()
		{
			bIsOpen = false;

			if (Document)
			{
				psd::DestroyDocument(Document, &Allocator);
				Document = nullptr;
			}

			if (OutputDocument.IsValid())
			{
				OutputDocument.Reset();
			}

			if (File.IsValid())
			{
				File->Close();
				File.Reset();
			}
		}

		bool IsOpen()
		{
			if (bIsOpen)
			{
				bool bIsValid = File.IsValid()
					&& File->GetSize() > 0
					&& Document != nullptr
					&& OutputDocument.IsValid();

				if (!bIsValid)
				{
					bIsOpen = false;
				}

				return bIsValid;
			}

			return false;
		}

		virtual bool Import(const TSharedPtr<FPSDFileImportVisitors>& InVisitors, const FPSDFileImporterOptions& InOptions) override
		{
			if (Open())
			{
				FReadContext ReadContext(Allocator, File.Get(), Document, FileReader, OutputDocument.Get(), InVisitors, InOptions);
				
				FDocumentReader DocumentReader;
				if (DocumentReader.Read(ReadContext))
				{
					InVisitors->OnImportComplete();	
				}

				return true;
			}

			return false;
		}

	private:
		TSharedPtr<IPSDFileReader> FileReader;

		FPsdAllocator Allocator;
		TSharedPtr<psd::NativeFile> File;
		psd::Document* Document = nullptr;
		bool bIsOpen = false;

		TSharedPtr<File::FPSDDocument> OutputDocument;

		FString FileName; // @todo: move to import options object
	};
}

namespace UE::PSDImporter
{
	TSharedRef<FPSDFileImporter> FPSDFileImporter::Make(const FString& InFileName)
	{
		TSharedRef<Private::FPSDFileImporterImpl> Instance = MakeShared<Private::FPSDFileImporterImpl>(InFileName);
		return StaticCastSharedRef<FPSDFileImporter>(Instance);
	}
}
