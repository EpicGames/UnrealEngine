// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDFileReader.h"

#include "ImageCoreUtils.h"
#include "Math/GuardedInt.h"
#include "PSDFileData.h"

namespace UE::PSDImporter::Internal
{
	int64 ReadBounds(FArchive& InAr, FIntRect& InBounds)
	{
		// Top, Left, Bottom, Right order
		InAr << InBounds.Min.Y;
		InAr << InBounds.Min.X;
		InAr << InBounds.Max.Y;
		InAr << InBounds.Max.X;

		return sizeof(int32) * 4;
	}

	template <>
	bool Read<uint16>(FMemoryView& InOutView, uint16& OutValue)
	{
		if (InOutView.GetSize() < 2)
		{
			OutValue = 0;
			return false;
		}

		OutValue = ReadBigEndian16(InOutView.GetData());
		InOutView.RightChopInline(2);
		return true;
	}

	bool SkipSection(FMemoryView& InOutView)
	{
		// PSD has several 32 bit sized sections we ignore.
		if (InOutView.GetSize() < 4)
		{
			return false;
		}

		const uint32 SectionSize = ReadBigEndian32(InOutView.GetData());
		InOutView.RightChopInline(SectionSize + 4);
		return InOutView.GetSize() != 0;
	}

	bool DecodeRLERow(const uint8* InRowSource, uint16 InRowSourceBytes, uint8* InOutputScanlineData, uint64 InOutputScanlineDataSize)
	{
		uint64 OutputByte = 0;
		uint32 SourceByteIndex = 0;

		while (SourceByteIndex < InRowSourceBytes)
		{
			int8 Code = static_cast<int8>(InRowSource[SourceByteIndex]);
			SourceByteIndex++;

			// Is it a repeat?
			if (Code == -128) // nop code for alignment.
			{
			}
			else if (Code < 0)
			{
				int32 Count = -static_cast<int32>(Code) + 1;

				if (OutputByte + Count > InOutputScanlineDataSize) // InOutputScanlineDataSize originates as int32 + int8 can't overflow a uint64
				{
					return false;
				}

				if (SourceByteIndex >= InRowSourceBytes)
				{
					return false;
				}

				uint8 Value = InRowSource[SourceByteIndex];
				SourceByteIndex++;

				FMemory::Memset(InOutputScanlineData + OutputByte, Value, Count);
				OutputByte += Count;
			}
			// Must be a run of literals then
			else
			{
				int32 Count = (int32)Code + 1;
				if (SourceByteIndex + Count > InRowSourceBytes) // int32 + int8, can't overflow.
				{
					return false;
				}

				FMemory::Memcpy(InOutputScanlineData + OutputByte, InRowSource + SourceByteIndex, Count);
				SourceByteIndex += Count;
				OutputByte += Count;
			}
		}

		// Confirm that we decoded the right number of bytes
		return OutputByte == InOutputScanlineDataSize;
	}

	bool ReadData(const FMutableMemoryView& Output, const FMemoryView Input, const File::FPSDHeader& InHeader)
	{
		// @todo: check for format support - bit depth, etc.

		// Double check to make sure this is a valid request
		if (!InHeader.IsValid())
		{
			return false;
		}

		if (Input.GetSize() <= sizeof(File::FPSDHeader)
			|| !FImageCoreUtils::IsImageImportPossible(InHeader.Width, InHeader.Height))
		{
			return false;
		}

		FMemoryView Current = Input;
		Current.RightChopInline(sizeof(File::FPSDHeader));

		const FGuardedInt64 GuardedPixelCount = FGuardedInt64(InHeader.Width) * InHeader.Height;

		if (GuardedPixelCount.InvalidOrLessOrEqual(0))
		{
			return false;
		}

		if (InHeader.Depth != 8 && InHeader.Depth != 16)
		{
			return false;
		}
		
		if (InHeader.NumChannels != 1 && InHeader.NumChannels != 3 && InHeader.NumChannels != 4)
		{
			return false;
		}

		const FGuardedInt64 OutputBytesNeeded = GuardedPixelCount * 4 * (InHeader.Depth / 8);

		if (OutputBytesNeeded.InvalidOrGreaterThan(Output.GetSize()))
		{
			return false;
		}

		// Skip Color LUT, Image Resource Section, and Layer/Mask Section
		if (SkipSection(Current) == false || //-V501
			SkipSection(Current) == false ||
			SkipSection(Current) == false)
		{
			return false;
		}

		// Get Compression Type
		uint16 CompressionType = 0;

		if (Read(Current, CompressionType) == false)
		{
			return false;
		}
		
		if (CompressionType != 0 && CompressionType != 1)
		{
			return false;
		}

		// Overflow checked above.
		const uint64 UncompressedScanLineSizePerChannel = InHeader.Width * (InHeader.Depth / 8);
		const uint64 OutputScanLineSize = UncompressedScanLineSizePerChannel * 4;
		const uint64 UncompressedPlaneSize = UncompressedScanLineSizePerChannel * InHeader.Height;

		// For copying alpha when the source doesn't have alpha information.
		constexpr uint8 OpaqueAlpha[2] = { 255, 255 };

		const uint16* RLERowTable[4] = {};
		const uint8* RLEPlaneSource[4] = {};
		TArray<uint8> RLETempScanlines[4];

		if (CompressionType == 1)
		{
			const uint64 RowTableBytesPerChannel = InHeader.Height * sizeof(uint16);
			if (Current.GetSize() < RowTableBytesPerChannel * InHeader.NumChannels)
			{
				return false;
			}

			FMemoryView RowTableSource = Current;
			RowTableSource.LeftInline(RowTableBytesPerChannel * InHeader.NumChannels);
			Current.RightChopInline(RowTableBytesPerChannel * InHeader.NumChannels);

			// We want the row tables for each plane, which means we have to decode them.
			uint64 CurrentOffset = 0;

			for (uint64 PlaneIdx = 0; PlaneIdx < static_cast<uint64>(InHeader.NumChannels); ++PlaneIdx)
			{
				RLETempScanlines[PlaneIdx].AddUninitialized(UncompressedScanLineSizePerChannel);
				RLERowTable[PlaneIdx] = static_cast<const uint16*>(RowTableSource.GetData());
				RowTableSource.RightChopInline(RowTableBytesPerChannel);

				// Save off where this plane's source is.
				RLEPlaneSource[PlaneIdx] = static_cast<const uint8*>(Current.GetData()) + CurrentOffset;

				for (uint64 RowIdx = 0; RowIdx < InHeader.Height; ++RowIdx)
				{
					// can't overflow: we're adding 16-bit uints and Info.Height is bounded by a 32-bit value, so sum fits in 48 bits
					CurrentOffset += ReadBigEndian16(&RLERowTable[PlaneIdx][RowIdx]);
				}

				// Now that we know the size, verify we have it.
				if (Current.GetSize() < CurrentOffset)
				{
					return false;
				}
			}
		}
		else
		{
			if (Current.GetSize() < UncompressedPlaneSize * InHeader.NumChannels) // overflow checked on function entry.
			{
				return false;
			}
		}

		uint8* OutputScanline = static_cast<uint8*>(Output.GetData());

		for (uint64 Row = 0; Row < static_cast<uint64>(InHeader.Height); Row++, OutputScanline += OutputScanLineSize)
		{
			const uint8* SourceScanLine[4] = {};
			uint64 AlphaMask = ~0ULL;

			// Init the source scanlines from the file data. For RLE we decode into a temp buffer,
			// otherwise we read directly. File size has already been validated.
			if (CompressionType == 0)
			{
				SourceScanLine[0] = static_cast<const uint8*>(Current.GetData()) + Row * UncompressedScanLineSizePerChannel;

				for (uint16 Channel = 1; Channel < InHeader.NumChannels; Channel++)
				{
					SourceScanLine[Channel] = SourceScanLine[Channel-1] + UncompressedPlaneSize;
				}
			}
			else
			{
				for (uint16 Channel = 0; Channel < InHeader.NumChannels; Channel++)
				{
					if (DecodeRLERow(RLEPlaneSource[Channel], ReadBigEndian16(&RLERowTable[Channel][Row]), RLETempScanlines[Channel].GetData(), RLETempScanlines[Channel].Num()) == false)
					{
						return false;
					}

					RLEPlaneSource[Channel] += ReadBigEndian16(&RLERowTable[Channel][Row]);
					SourceScanLine[Channel] = RLETempScanlines[Channel].GetData();
				}
			}

			// If we don't have all 4 channels, set up the scanlines to valid data.
			if (InHeader.NumChannels == 1)
			{
				SourceScanLine[1] = SourceScanLine[0];
				SourceScanLine[2] = SourceScanLine[0];
				SourceScanLine[3] = OpaqueAlpha;
				AlphaMask = 0;
			}
			else if (InHeader.NumChannels == 3)
			{
				SourceScanLine[3] = OpaqueAlpha;
				AlphaMask = 0;
			}
			else if (InHeader.NumChannels == 4)
			{
				AlphaMask = ~0ULL;
			}

			// Do the plane interleaving.
			if (InHeader.Depth == 8)
			{
				FColor* ScanLine8 = reinterpret_cast<FColor*>(OutputScanline);

				for (uint64 X = 0; X < InHeader.Width; X++)
				{
					ScanLine8[X].R = SourceScanLine[0][X];
					ScanLine8[X].G = SourceScanLine[1][X];
					ScanLine8[X].B = SourceScanLine[2][X];
					ScanLine8[X].A = SourceScanLine[3][X & AlphaMask];
				}
			}
			else if (InHeader.Depth == 16)
			{
				const uint16* SourceScanLineR16 = reinterpret_cast<const uint16*>(SourceScanLine[0]);
				const uint16* SourceScanLineG16 = reinterpret_cast<const uint16*>(SourceScanLine[1]);
				const uint16* SourceScanLineB16 = reinterpret_cast<const uint16*>(SourceScanLine[2]);
				const uint16* SourceScanLineA16 = reinterpret_cast<const uint16*>(SourceScanLine[3]);

				uint16* ScanLine16 = reinterpret_cast<uint16*>(OutputScanline);

				for (uint64 X = 0; X < InHeader.Width; X++)
				{
					ScanLine16[4*X + 0] = ReadBigEndian16(&SourceScanLineR16[X]);
					ScanLine16[4*X + 1] = ReadBigEndian16(&SourceScanLineG16[X]);
					ScanLine16[4*X + 2] = ReadBigEndian16(&SourceScanLineB16[X]);
					ScanLine16[4*X + 3] = ReadBigEndian16(&SourceScanLineA16[X & AlphaMask]);
				}
			}
		} // end each row.

		// Success!
		return true;
	}
}
