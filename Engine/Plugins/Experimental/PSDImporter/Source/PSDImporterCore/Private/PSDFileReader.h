// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"

namespace UE::PSDImporter
{
	struct FPSDFileHeader;
}

namespace UE::PSDImporter::Internal
{
	int64 ReadBounds(FArchive& InAr, FIntRect& InBounds);

	template <typename DataType = uint16>
	DataType ReadBigEndian16(const void* InPtr)
	{
		const uint8* Data = static_cast<const uint8*>(InPtr);
		return static_cast<DataType>(
			(static_cast<uint16>(Data[0]) << 8) 
			+ (static_cast<uint16>(Data[1]) << 0));
	}

	template <typename DataType = uint32>
	DataType ReadBigEndian32(const void* InPtr)
	{
		const uint8* Data = static_cast<const uint8*>(InPtr);
		return static_cast<DataType>(
			(static_cast<uint32>(Data[0]) << 24)
			+ (static_cast<uint32>(Data[1]) << 16)
			+ (static_cast<uint32>(Data[2]) << 8)
			+ (static_cast<uint32>(Data[3]) << 0));
	}

	template <typename DataType>
	bool Read(FMemoryView& InOutView, DataType& OutValue)
	{
		unimplemented();
		return false;
	}
	
	template <>
	bool Read(FMemoryView& InOutView, uint16& OutValue);

    bool SkipSection(FMemoryView& InOutView);

    // Decodes the raw data for a row - this is the same independent of the scanline format.
    bool DecodeRLERow(const uint8* InRowSource, uint16 InRowSourceBytes, uint8* InOutputScanlineData, uint64 InOutputScanlineDataSize);

    bool ReadData(const FMutableMemoryView& InOutput, FMemoryView InInput, const FPSDFileHeader& InHeader);
}
