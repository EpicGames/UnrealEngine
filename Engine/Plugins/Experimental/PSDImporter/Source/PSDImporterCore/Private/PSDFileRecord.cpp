// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDFileRecord.h"

namespace UE::PSDImporter::File
{
	FPSDLayerRecord::~FPSDLayerRecord()
	{
		if (!Channels.IsEmpty())
		{
			for (FPSDChannelInformation*& Channel : Channels)
			{
				delete Channel;
				Channel = nullptr;
			}

			Channels.Reset();
		}
	}

	bool FPSDLayerRecord::operator==(const FPSDLayerRecord& InOther) const
	{
		return Index == InOther.Index
			&& LayerName == InOther.LayerName;
	}

	bool FPSDLayerRecord::operator!=(const FPSDLayerRecord& InOther) const
	{
		return !(*this == InOther);
	}

	bool FPSDLayerRecord::operator<(const FPSDLayerRecord& InOther) const
	{
		return Index < InOther.Index;
	}

	bool FPSDLayerRecord::operator<=(const FPSDLayerRecord& InOther) const
	{
		return operator<(InOther) || operator==(InOther);
	}

	uint32 GetTypeHash(const FPSDLayerRecord& Value)
	{
		return HashCombineFast(Value.Index, GetTypeHash(Value.LayerName));
	}
}
