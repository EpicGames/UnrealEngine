// Copyright Epic Games, Inc. All Rights Reserved.

#include "Readers/DocumentReader.h"

#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"
#include "PSDFileData.h"
#include "PSDFileDocument.h"
#include "PsdDocument.h"
#include "PsdFile.h"
#include "PsdImageDataSection.h"
#include "PsdImageResourcesSection.h"
#include "PsdNativeFile.h"
#include "PsdParseImageDataSection.h"
#include "PsdParseImageResourcesSection.h"
#include "PsdThumbnail.h"
#include "Readers/LayerReader.h"
#include "Readers/ReaderShared.h"

namespace UE::PSDImporter::Private
{
	bool FDocumentReader::Read(FReadContext& InContext)
	{
		check(InContext.Document);

		InContext.Document2->Header.Signature   = 0;
		InContext.Document2->Header.Version     = 0;
		InContext.Document2->Header.NumChannels = InContext.Document->channelCount;
		InContext.Document2->Header.Height      = InContext.Document->height;
		InContext.Document2->Header.Width       = InContext.Document->width;
		InContext.Document2->Header.Depth       = InContext.Document->bitsPerChannel;
		InContext.Document2->Header.Mode        = static_cast<File::EPSDColorMode>(InContext.Document->colorMode);

		for (int32 Index = 0; Index < 6; ++Index)
		{
			InContext.Document2->Header.Pad[Index] = 0;
		}

		InContext.Visitors->OnImportHeader(InContext.Document2->Header);

		// Sections:

		// 1. ColorModeData: unsupported by lib 

		// 2. ImageResources
		if (psd::ImageResourcesSection* ImageResources = psd::ParseImageResourcesSection(InContext.Document, InContext.File, &InContext.Allocator))
		{
			// Thumbnail
			if (psd::Thumbnail* Thumbnail = ImageResources->thumbnail)
			{
				IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));

				FImage ThumbnailImage;
				if (ImageWrapperModule.DecompressImage(Thumbnail->binaryJpeg, Thumbnail->binaryJpegSize, ThumbnailImage))
				{
					// @todo: add to document
					// @todo: implement, use code from LayerReader
					// SaveImage(ThumbnailImage, TEXT("C:\\Temp\\Thumb.png"));
				}
			}

			// @todo: get other supported data here, ie. guides
			{
				
			}

			psd::DestroyImageResourcesSection(ImageResources, &InContext.Allocator);
		}

		// 3. Layer and Mask information
		FLayersReader LayersReader;
		LayersReader.Read(InContext);

		// 4. Image Data: merged image, only available with "maximize compatibility" option enabled in PSD
		// @todo: implement, use code from LayerReader
		if (psd::ImageDataSection* ImageData = psd::ParseImageDataSection(InContext.Document, InContext.File, &InContext.Allocator))
		{
			psd::DestroyImageDataSection(ImageData, &InContext.Allocator);
		}

		return true;
	}
}
