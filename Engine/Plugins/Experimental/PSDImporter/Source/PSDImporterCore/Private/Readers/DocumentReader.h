// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReaderShared.h"
#include "Templates/SharedPointerFwd.h"

namespace psd
{
	struct Document;
}

namespace UE::PSDImporter::Private
{
	class FDocumentReader
	{
	public:
		bool Read(FReadContext& InContext);
	};
}
