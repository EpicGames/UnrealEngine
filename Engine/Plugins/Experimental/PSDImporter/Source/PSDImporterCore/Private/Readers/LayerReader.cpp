// Copyright Epic Games, Inc. All Rights Reserved.

#include "Readers/LayerReader.h"

#include "Async/Async.h"
#include "Psd.h"
#include "PsdBlendMode.h"
#include "PsdChannel.h"
#include "PsdChannelType.h"
#include "PSDFileDocument.h"
#include "PSDFileRecord.h"
#include "PsdLayer.h"
#include "PsdLayerMask.h"
#include "PsdLayerMaskSection.h"
#include "PsdLayerType.h"
#include "PsdParseLayerMaskSection.h"
#include "PsdVectorMask.h"

namespace UE::PSDImporter::Private
{
	EPSDBlendMode ConvertBlendMode(uint32 InPsdBlendMode)
	{
		static TMap<uint32, EPSDBlendMode> Lookup = {
			{ psd::blendMode::PASS_THROUGH, EPSDBlendMode::PassThrough },
			{ psd::blendMode::NORMAL, EPSDBlendMode::Normal },
			{ psd::blendMode::DISSOLVE, EPSDBlendMode::Dissolve },
			
			{ psd::blendMode::DARKEN, EPSDBlendMode::Darken },
			{ psd::blendMode::MULTIPLY, EPSDBlendMode::Multiply },
			{ psd::blendMode::COLOR_BURN, EPSDBlendMode::ColorBurn },
			{ psd::blendMode::LINEAR_BURN, EPSDBlendMode::LinearBurn },
			{ psd::blendMode::DARKER_COLOR, EPSDBlendMode::DarkerColor },
			
			{ psd::blendMode::LIGHTEN, EPSDBlendMode::Lighten },
			{ psd::blendMode::SCREEN, EPSDBlendMode::Screen },
			{ psd::blendMode::COLOR_DODGE, EPSDBlendMode::ColorDodge },
			{ psd::blendMode::LINEAR_DODGE, EPSDBlendMode::LinearDodge },
			{ psd::blendMode::LIGHTER_COLOR, EPSDBlendMode::LighterColor },
						
			{ psd::blendMode::OVERLAY, EPSDBlendMode::Overlay },
			{ psd::blendMode::SOFT_LIGHT, EPSDBlendMode::SoftLight },
			{ psd::blendMode::HARD_LIGHT, EPSDBlendMode::HardLight },
			{ psd::blendMode::VIVID_LIGHT, EPSDBlendMode::VividLight },
			{ psd::blendMode::LINEAR_LIGHT, EPSDBlendMode::LinearLight },
			{ psd::blendMode::PIN_LIGHT, EPSDBlendMode::PinLight },
			{ psd::blendMode::HARD_MIX, EPSDBlendMode::HardMix },
			
			{ psd::blendMode::DIFFERENCE, EPSDBlendMode::Difference },
			{ psd::blendMode::EXCLUSION, EPSDBlendMode::Exclusion },
			{ psd::blendMode::SUBTRACT, EPSDBlendMode::Subtract },
			{ psd::blendMode::DIVIDE, EPSDBlendMode::Divide },
						
			{ psd::blendMode::HUE, EPSDBlendMode::Hue },
			{ psd::blendMode::SATURATION, EPSDBlendMode::Saturation },
			{ psd::blendMode::COLOR, EPSDBlendMode::Color },
			{ psd::blendMode::LUMINOSITY, EPSDBlendMode::Luminosity },
		};

		if (EPSDBlendMode* Found = Lookup.Find(psd::blendMode::KeyToEnum(InPsdBlendMode)))
		{
			return *Found;
		}

		return EPSDBlendMode::Unknown;
	}

	template<typename InScanlineType>
	InScanlineType GetPixelFromScanline(const InScanlineType* InInputScanline, const uint32 InPixel)
	{
		if (InInputScanline)
		{
			return InInputScanline[InPixel];
		}

		return 0;
	};

	template <typename DataType, int Depth>
	TFuture<FImage> ReadRGBAInternal(FReadContext& InContext, psd::Layer* InLayer)
	{
		psd::ExtractLayer(InContext.Document, InContext.File, &InContext.Allocator, InLayer);
		
		return Async(EAsyncExecution::ThreadPool, [&InContext, InLayer]()
		{
			const uint32 RedIdx = FindChannelIdx(InLayer, psd::channelType::R);
			const uint32 GreenIdx = FindChannelIdx(InLayer, psd::channelType::G);
			const uint32 BlueIdx = FindChannelIdx(InLayer, psd::channelType::B);
			const uint32 AlphaIdx = FindChannelIdx(InLayer, psd::channelType::TRANSPARENCY_MASK);

			DataType* CanvasData[4] = {};

			const bool bHasRGB = (RedIdx != INDEX_NONE) && (GreenIdx != INDEX_NONE) && (BlueIdx != INDEX_NONE);

			if (!bHasRGB)
			{
				return FImage();
			}

			uint8 NumChannels = 0;
			
			const uint32 Width = InContext.Options.bResizeLayersToDocument ? InContext.Document->width : InLayer->right - InLayer->left;
			const uint32 Height = InContext.Options.bResizeLayersToDocument ? InContext.Document->height : InLayer->bottom - InLayer->top;

			if (InContext.Options.bResizeLayersToDocument)
			{
				CanvasData[0] = ExpandChannelToCanvas<DataType>(&InContext.Allocator, InLayer, InLayer->channels[RedIdx].data, Width, Height);
				CanvasData[1] = ExpandChannelToCanvas<DataType>(&InContext.Allocator, InLayer, InLayer->channels[GreenIdx].data, Width, Height);
				CanvasData[2] = ExpandChannelToCanvas<DataType>(&InContext.Allocator, InLayer, InLayer->channels[BlueIdx].data, Width, Height);
			}
			else
			{
				CanvasData[0] = static_cast<DataType*>(InLayer->channels[RedIdx].data);
				CanvasData[1] = static_cast<DataType*>(InLayer->channels[GreenIdx].data);
				CanvasData[2] = static_cast<DataType*>(InLayer->channels[BlueIdx].data);
			}

			NumChannels = 3;

			const bool bHasAlpha = AlphaIdx != INDEX_NONE;

			if (bHasAlpha)
			{
				if (InContext.Options.bResizeLayersToDocument)
				{
					CanvasData[3] = ExpandChannelToCanvas<DataType>(&InContext.Allocator, InLayer, InLayer->channels[AlphaIdx].data, Width, Height);
				}
				else
				{
					CanvasData[3] = static_cast<DataType*>(InLayer->channels[AlphaIdx].data);
				}

				NumChannels = 4;
			}

			FImage OutputImage;

			if constexpr (Depth == 8)
			{
				OutputImage.Init(Width, Height, ERawImageFormat::BGRA8);
			}
			else if constexpr (Depth == 16)
			{
				OutputImage.Init(Width, Height, ERawImageFormat::RGBA16);
			}
			else if constexpr (Depth == 32)
			{
				OutputImage.Init(Width, Height, ERawImageFormat::RGBA32F);
			}

			// Copy planes to (interleaved) image
			{
				// Max 32 bits
				constexpr uint8 One[4] = {255, 255, 255, 255};
				
				const uint32 InputScanlineSizePerChannel = Width * (Depth / 8);
				const uint32 OutputScanlineSize = Width * sizeof(DataType) * 4;
				
				uint8* OutputScanline = OutputImage.RawData.GetData();

				for (uint32 RowIdx = 0; RowIdx < Height; ++RowIdx, OutputScanline += OutputScanlineSize)
				{
					const uint32 OutputScanLineEnd = RowIdx * OutputScanlineSize;

					if (OutputScanLineEnd >= OutputImage.RawData.Num())
					{
						break;
					}

					const uint8* InputScanline[4] = {};
					uint32 AlphaMask = -1;

					if (NumChannels == 3)
					{
						InputScanline[3] = One;
						AlphaMask = 0;
					}

					for (uint16 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
					{
						InputScanline[ChannelIdx] = reinterpret_cast<const uint8*>(CanvasData[ChannelIdx]) + (RowIdx * InputScanlineSizePerChannel);
					}

					if constexpr (Depth == 8)
					{
						FColor* Scanline8 = reinterpret_cast<FColor*>(OutputScanline);

						for (uint32 PixelX = 0; PixelX < Width; ++PixelX)
						{
							Scanline8[PixelX].R = GetPixelFromScanline<uint8>(InputScanline[0], PixelX);
							Scanline8[PixelX].G = GetPixelFromScanline<uint8>(InputScanline[1], PixelX);
							Scanline8[PixelX].B = GetPixelFromScanline<uint8>(InputScanline[2], PixelX);
							Scanline8[PixelX].A = GetPixelFromScanline<uint8>(InputScanline[3], PixelX) & AlphaMask;
						}
					}
					else if constexpr (Depth == 16)
					{
						const uint16* InputScanlineR16 = reinterpret_cast<const uint16*>(InputScanline[0]);
						const uint16* InputScanlineG16 = reinterpret_cast<const uint16*>(InputScanline[1]);
						const uint16* InputScanlineB16 = reinterpret_cast<const uint16*>(InputScanline[2]);
						const uint16* InputScanlineA16 = reinterpret_cast<const uint16*>(InputScanline[3]);

						struct FColorUInt16
						{
							uint16 R;
							uint16 G;
							uint16 B;
							uint16 A;
						};

						FColorUInt16* Scanline16 = reinterpret_cast<FColorUInt16*>(OutputScanline);

						for (uint32 PixelX = 0; PixelX < Width; ++PixelX)
						{
							Scanline16[PixelX].R = GetPixelFromScanline<uint16>(InputScanlineR16, PixelX);
							Scanline16[PixelX].G = GetPixelFromScanline<uint16>(InputScanlineG16, PixelX);
							Scanline16[PixelX].B = GetPixelFromScanline<uint16>(InputScanlineB16, PixelX);
							Scanline16[PixelX].A = GetPixelFromScanline<uint16>(InputScanlineA16, PixelX) & AlphaMask;
						}
					}
					else if constexpr (Depth == 32)
					{
						const uint32* InputScanlineR32 = reinterpret_cast<const uint32*>(InputScanline[0]);
						const uint32* InputScanlineG32 = reinterpret_cast<const uint32*>(InputScanline[1]);
						const uint32* InputScanlineB32 = reinterpret_cast<const uint32*>(InputScanline[2]);
						const uint32* InputScanlineA32 = reinterpret_cast<const uint32*>(InputScanline[3]);

						struct FColorF32
						{
							float R;
							float G;
							float B;
							float A;
						};
						
						FColorF32* Scanline32 = reinterpret_cast<FColorF32*>(OutputScanline);
						constexpr float MaxValue = 4294967296.f;

						for (uint32 PixelX = 0; PixelX < Width; ++PixelX)
						{
							Scanline32[PixelX].R = static_cast<float>(GetPixelFromScanline<uint32>(InputScanlineR32, PixelX)) / MaxValue;
							Scanline32[PixelX].G = static_cast<float>(GetPixelFromScanline<uint32>(InputScanlineG32, PixelX)) / MaxValue;
							Scanline32[PixelX].B = static_cast<float>(GetPixelFromScanline<uint32>(InputScanlineB32, PixelX)) / MaxValue;
							Scanline32[PixelX].A = static_cast<float>(GetPixelFromScanline<uint32>(InputScanlineA32, PixelX) & AlphaMask) / MaxValue;
						}
					}
				}
			}

			if (InContext.Options.bResizeLayersToDocument)
			{
				InContext.Allocator.Free(CanvasData[0]);
				InContext.Allocator.Free(CanvasData[1]);
				InContext.Allocator.Free(CanvasData[2]);
				InContext.Allocator.Free(CanvasData[3]);
			}

			return OutputImage;
		});
	}

	template <typename DataType, int Depth>
	TFuture<FImage> ReadMaskInternal(FReadContext& InContext, psd::Layer* InLayer, ELayerMaskType InMaskType)
	{
		psd::ExtractLayer(InContext.Document, InContext.File, &InContext.Allocator, InLayer);

		return Async(EAsyncExecution::ThreadPool, [&InContext, InLayer, InMaskType]()
			{
				DataType* CanvasData = nullptr;
				uint32 Width;
				uint32 Height;

				if (InContext.Options.bResizeLayersToDocument)
				{
					Width = InContext.Options.bResizeLayersToDocument;
					Height = InContext.Options.bResizeLayersToDocument;

					switch (InMaskType)
					{
						case ELayerMaskType::LayerMask:
							CanvasData = ExpandChannelToCanvas<DataType>(&InContext.Allocator, InLayer, InLayer->layerMask->data, Width, Height);
							break;

						case ELayerMaskType::VectorMask:
							CanvasData = ExpandChannelToCanvas<DataType>(&InContext.Allocator, InLayer, InLayer->vectorMask->data, Width, Height);
							break;

						default:
							return FImage();
					}
				}
				else
				{
					switch (InMaskType)
					{
						case ELayerMaskType::LayerMask:
							Width = InLayer->layerMask->right - InLayer->layerMask->left;
							Height = InLayer->layerMask->bottom - InLayer->layerMask->top;
							CanvasData = static_cast<DataType*>(InLayer->layerMask->data);
							break;

						case ELayerMaskType::VectorMask:
							Width = InLayer->vectorMask->right - InLayer->vectorMask->left;
							Height = InLayer->vectorMask->bottom - InLayer->vectorMask->top;
							CanvasData = static_cast<DataType*>(InLayer->vectorMask->data);
							break;

						default:
							return FImage();
					}
				}

				FImage OutputImage;
				OutputImage.Init(Width, Height, ERawImageFormat::G8);

				// Copy planes to (interleaved) image
				{
					auto GetPixelFromScanline = []<typename InScanlineType>(const InScanlineType* InInputScanline, const uint32 InPixel) -> InScanlineType
					{
						if (InInputScanline)
						{
							return InInputScanline[InPixel];
						}

						return 0;
					};

					// Max 8 bits
					constexpr uint8 One = 255;

					const uint32 InputScanlineSize = Width * (Depth / 8);
					const uint32 OutputScanlineSize = Width * sizeof(DataType);

					uint8* OutputScanline = OutputImage.RawData.GetData();

					for (uint32 RowIdx = 0; RowIdx < Height; ++RowIdx, OutputScanline += OutputScanlineSize)
					{
						const uint8* InputScanline = nullptr;
						uint32 AlphaMask = ~0U;

						InputScanline = reinterpret_cast<const uint8*>(CanvasData) + (RowIdx * InputScanlineSize);

						if constexpr (Depth == 8)
						{
							for (uint32 PixelX = 0; PixelX < Width; ++PixelX)
							{
								OutputScanline[PixelX] = GetPixelFromScanline(InputScanline, PixelX);
							}
						}
						else if constexpr (Depth == 16)
						{
							const uint16* InputScanline16 = reinterpret_cast<const uint16*>(InputScanline);
							uint16* Scanline16 = reinterpret_cast<uint16*>(OutputScanline);

							for (uint32 PixelX = 0; PixelX < Width; ++PixelX)
							{
								Scanline16[PixelX] = GetPixelFromScanline(InputScanline16, PixelX);
							}
						}
						else if constexpr (Depth == 32)
						{
							const uint32* InputScanline32 = reinterpret_cast<const uint32*>(InputScanline[0]);
							uint32* Scanline32 = reinterpret_cast<uint32*>(OutputScanline);

							for (uint32 PixelX = 0; PixelX < Width; ++PixelX)
							{
								Scanline32[PixelX] = GetPixelFromScanline(InputScanline32, PixelX);
							}
						}
					}
				}

				if (InContext.Options.bResizeLayersToDocument)
				{
					InContext.Allocator.Free(CanvasData);
				}

				return OutputImage;
			});
	}

	File::FPSDLayerRecord* FLayerReader::Read(FReadContext& InContext, psd::Layer* InLayer, const int32 InLayerIdx, const File::FPSDLayerRecord* InParentLayer)
	{
		File::FPSDLayerRecord* OutputLayer = new File::FPSDLayerRecord();
		OutputLayer->Index       = InLayerIdx;
		OutputLayer->Bounds      = FIntRect(InLayer->left, InLayer->top, InLayer->right, InLayer->bottom);
		OutputLayer->NumChannels = InLayer->channelCount;
		OutputLayer->BlendMode   = ConvertBlendMode(InLayer->blendModeKey);
		OutputLayer->Opacity     = InLayer->opacity;
		OutputLayer->Clipping    = InLayer->clipping;
		OutputLayer->Flags       = InLayer->isVisible ? File::EPSDLayerFlags::Visible : File::EPSDLayerFlags::None;
		OutputLayer->bIsGroup    = false;
		OutputLayer->LayerName   = GetLayerName(InLayer);

		if (InLayer->layerMask)
		{
			OutputLayer->MaskBounds = FIntRect(InLayer->layerMask->left, InLayer->layerMask->top, InLayer->layerMask->right, InLayer->layerMask->bottom);
			OutputLayer->MaskDefaultValue = static_cast<float>(InLayer->layerMask->defaultColor) / 255.f;
		}
		else if (InLayer->vectorMask)
		{
			OutputLayer->MaskBounds = FIntRect(InLayer->vectorMask->left, InLayer->vectorMask->top, InLayer->vectorMask->right, InLayer->vectorMask->bottom);
			OutputLayer->MaskDefaultValue = static_cast<float>(InLayer->vectorMask->defaultColor) / 255.f;
		}
		else
		{
			OutputLayer->MaskBounds = FIntRect();
			OutputLayer->MaskDefaultValue = 1.f;
		}

		TFunction<TFuture<FImage>()> DataReader = [&InContext, InLayer]() -> TFuture<FImage>
			{			
				FLayerReader LayerReader;
				return LayerReader.ReadLayerData(InContext, InLayer);
			};

		TFunction<TFuture<FImage>()> MaskReader = [&InContext, InLayer]() -> TFuture<FImage>
			{
				FLayerReader LayerReader;
				return LayerReader.ReadMaskData(InContext, InLayer);
			};

		InContext.Visitors->OnImportLayer(*OutputLayer, InParentLayer, DataReader, MaskReader);

		return OutputLayer;
	}

	TFuture<FImage> FLayerReader::ReadLayerData(FReadContext& InContext, psd::Layer* InLayer)
	{
		if (InContext.Document->bitsPerChannel == 8)
		{
			return ReadRGBAInternal<uint8, 8>(InContext, InLayer);
		}

		if (InContext.Document->bitsPerChannel == 16)
		{
			return ReadRGBAInternal<uint16, 16>(InContext, InLayer);
		}

		if (InContext.Document->bitsPerChannel == 32)
		{
			return ReadRGBAInternal<float, 32>(InContext, InLayer);
		}

		return MakeFulfilledPromise<FImage>().GetFuture();
	}

	TFuture<FImage> FLayerReader::ReadMaskData(FReadContext& InContext, psd::Layer* InLayer)
	{
		ELayerMaskType MaskType;

		if (InLayer->layerMask)
		{
			MaskType = ELayerMaskType::LayerMask;
		}
		else if (InLayer->vectorMask)
		{
			MaskType = ELayerMaskType::VectorMask;
		}
		else
		{
			return MakeFulfilledPromise<FImage>().GetFuture();
		}

		if (InContext.Document->bitsPerChannel == 8)
		{
			return ReadMaskInternal<uint8, 8>(InContext, InLayer, MaskType);
		}

		if (InContext.Document->bitsPerChannel == 16)
		{
			return ReadMaskInternal<uint16, 16>(InContext, InLayer, MaskType);
		}

		if (InContext.Document->bitsPerChannel == 32)
		{
			return ReadMaskInternal<float, 32>(InContext, InLayer, MaskType);
		}

		return MakeFulfilledPromise<FImage>().GetFuture();
	}

	TFuture<bool> FLayersReader::Read(Private::FReadContext& InContext)
	{
		File::FPSDDocument* Document = InContext.Document2;
		if (psd::LayerMaskSection* LayerMasks = psd::ParseLayerMaskSection(InContext.Document, InContext.File, &InContext.Allocator)) //-V1051
		{
			File::FPSDLayerAndMaskInformation& LayersAndMasks = Document->LayerAndMaskInformation;
			LayersAndMasks.NumLayers = LayerMasks->layerCount;
			LayersAndMasks.bHasTransparencyMask = LayerMasks->hasTransparencyMask;

			InContext.Visitors->OnImportLayers(LayersAndMasks);

			FReadContext Context(InContext, LayerMasks);
			FLayerData LayerData{0, nullptr, LayersAndMasks.Layers};
			ReadLayers(Context, LayerData,
				[](psd::Layer* InputLayer, File::FPSDLayerRecord* OutputLayer)
				{
					return true;
				});

			psd::DestroyLayerMaskSection(LayerMasks, &InContext.Allocator);
		}

		return MakeFulfilledPromise<bool>(true).GetFuture();
	}

	int32 FLayersReader::ReadLayers(
		FReadContext& InContext,
		FLayerData& InOutLayerData,
		const TFunctionRef<bool(psd::Layer*, File::FPSDLayerRecord*)>& InLayerVisitorFunc)
	{
		int32 LayerIdx = InOutLayerData.LayerIdx;
		while (LayerIdx < static_cast<int32>(InContext.LayerMasks->layerCount))
		{
			psd::Layer* InputLayer = &InContext.LayerMasks->layers[LayerIdx];
			psd::layerType::Enum LayerType = static_cast<psd::layerType::Enum>(InputLayer->type);

			bool bShouldContinue = true;			
			if (LayerType == psd::layerType::SECTION_DIVIDER)
			{
				File::FPSDLayerRecord* OutputLayer = ReadGroup(InContext, InputLayer, InOutLayerData);
				bShouldContinue = InLayerVisitorFunc(InputLayer, OutputLayer); 
				InOutLayerData.LayerRecords.Emplace(MoveTemp(OutputLayer));
				LayerIdx = InOutLayerData.LayerIdx;
			}
			else if (LayerType == psd::layerType::OPEN_FOLDER
				|| LayerType == psd::layerType::CLOSED_FOLDER)
			{
				break;
			}
			else if (LayerType == psd::layerType::ANY)
			{
				File::FPSDLayerRecord* OutputLayer = LayerReader.Read(InContext, InputLayer, LayerIdx, InOutLayerData.ParentLayer);
				bShouldContinue = InLayerVisitorFunc(InputLayer, OutputLayer);
				InOutLayerData.LayerRecords.Emplace(MoveTemp(OutputLayer));
			}

			if (!bShouldContinue)
			{
				break;
			}

			++LayerIdx;
		}

		return LayerIdx;
	}

	File::FPSDLayerRecord* FLayersReader::ReadGroup(FReadContext& InContext, const psd::Layer* InLayer, FLayerData& InOutLayerData)
	{
		File::FPSDLayerRecord* OutputLayerGroup = new File::FPSDLayerRecord();
		OutputLayerGroup->Index       = InOutLayerData.LayerIdx;
		OutputLayerGroup->Bounds      = FIntRect(InLayer->left, InLayer->top, InLayer->right, InLayer->bottom);
		OutputLayerGroup->NumChannels = InLayer->channelCount;
		OutputLayerGroup->BlendMode   = ConvertBlendMode(InLayer->blendModeKey);
		OutputLayerGroup->Opacity     = InLayer->opacity;
		OutputLayerGroup->Clipping    = InLayer->clipping;
		OutputLayerGroup->Flags       = File::EPSDLayerFlags::Visible;
		OutputLayerGroup->bIsGroup    = true;
		OutputLayerGroup->LayerName   = GetLayerName(InLayer->parent);

		++InOutLayerData.LayerIdx;

		InOutLayerData.LayerIdx = ReadLayers(InContext, InOutLayerData, [](const psd::Layer* InputLayer, File::FPSDLayerRecord* OutputLayer)
		{
			return true;
		});

		InContext.Visitors->OnImportLayer(*OutputLayerGroup, InOutLayerData.ParentLayer, nullptr, nullptr);

		return OutputLayerGroup;
	}
}
