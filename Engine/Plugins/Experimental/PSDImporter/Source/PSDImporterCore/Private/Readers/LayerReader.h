// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PSDFileData.h"
#include "ReaderShared.h"
#include "Templates/SharedPointerFwd.h"

enum class EPSDBlendMode : uint8;

namespace psd
{
	struct LayerMaskSection;
	struct Layer;
}

namespace UE::PSDImporter::Private
{
	enum class ELayerMaskType : uint8
	{
		LayerMask,
		VectorMask
	};

	static EPSDBlendMode ConvertBlendMode(uint32 InPsdBlendMode);

	class FLayerReader
	{
	public:
		using FInputType = psd::Layer;

		File::FPSDLayerRecord* Read(FReadContext& InContext, psd::Layer* InLayer, const int32 InLayerIdx, const File::FPSDLayerRecord* InParentLayer);
		TFuture<FImage> ReadLayerData(FReadContext& InContext, psd::Layer* InLayer);
		TFuture<FImage> ReadMaskData(FReadContext& InContext, psd::Layer* InLayer);
	};

	class FLayersReader
	{
		struct FReadContext : Private::FReadContext
		{
			FReadContext(const Private::FReadContext& InBaseContext, psd::LayerMaskSection* InLayerMasks)
				: Private::FReadContext(InBaseContext)
				, LayerMasks(InLayerMasks)
			{
			}

			psd::LayerMaskSection* LayerMasks;
		};

		struct FLayerData
		{
			int32 LayerIdx;
			File::FPSDLayerRecord* ParentLayer = nullptr;
			TSet<File::FPSDLayerRecord*>& LayerRecords;
		};
		
	public:
		using FInputType = psd::LayerMaskSection;
		
		TFuture<bool> Read(Private::FReadContext& InContext);

	private:
		int32 ReadLayers(
			FReadContext& InContext,
			FLayerData& InOutLayerData, 
			const TFunctionRef<bool(psd::Layer*, File::FPSDLayerRecord*)>& InLayerVisitorFunc = [](psd::Layer*, File::FPSDLayerRecord*) { return true; }
		);

		File::FPSDLayerRecord* ReadGroup(FReadContext& InContext, const psd::Layer* InLayer, FLayerData& InOutLayerData);

	private:
		FLayerReader LayerReader;
	};
}
