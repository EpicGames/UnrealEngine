// Copyright Epic Games, Inc. All Rights Reserved.

#include "Readers/ReaderShared.h"

#include "ImageUtils.h"
#include "PsdLayer.h"
#include "PsdChannel.h"

namespace UE::PSDImporter::Private
{
	bool SaveImage(const FImage& InImage, const FString& InFilePath)
	{
		return FImageUtils::SaveImageAutoFormat(*InFilePath, InImage);
	}

	uint32 FindChannelIdx(const psd::Layer* InLayer, int16 InChannelType)
	{
		check(InLayer);
		
		for (uint32 ChannelIdx = 0; ChannelIdx < InLayer->channelCount; ++ChannelIdx)
		{
			psd::Channel* Channel = &InLayer->channels[ChannelIdx];
			if (Channel->data && Channel->type == InChannelType)
			{
				return ChannelIdx;
			}
		}
		
		return INDEX_NONE;
	}

	FString GetLayerName(const psd::Layer* InLayer)
	{
		FString LayerName;
		if (InLayer->utf16Name)
		{
			LayerName = WCHAR_TO_TCHAR(InLayer->utf16Name);
		}
		else
		{
			LayerName = InLayer->name.c_str();
		}

		return LayerName;
	}
}
