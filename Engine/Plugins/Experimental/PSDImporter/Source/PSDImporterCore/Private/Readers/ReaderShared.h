// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PSDFileImport.h"
#include "Psd.h"
#include "PsdAllocator.h"
#include "PsdDocument.h"
#include "PsdLayer.h"
#include "PsdLayerCanvasCopy.h"
#include "PsdNativeFile.h"

namespace UE::PSDImporter::Private
{
	class FPsdAllocator
		: public psd::Allocator
	{
	public:
		virtual void* DoAllocate(size_t InSize, size_t InAlignment) override
		{
			return FMemory::Malloc(InSize, InAlignment);
		}
		
		virtual void DoFree(void* InPtr) override
		{
			FMemory::Free(InPtr);
		}
	};

	struct FReadContext
	{
		FReadContext(
			psd::Allocator& InAllocator,
			psd::NativeFile* InFile,
			psd::Document* InDocument,
			const TSharedPtr<IPSDFileReader>& InReader,
			File::FPSDDocument* InDocument2,
			const TSharedPtr<FPSDFileImportVisitors>& InVisitors,
			const FPSDFileImporterOptions& InOptions)
			: Allocator(InAllocator)
			, File(InFile)
			, Document(InDocument)
			, FileReader(InReader)
			, Document2(InDocument2)
			, Visitors(InVisitors)
			, Options(InOptions)
		{
		}

		psd::Allocator& Allocator;
		psd::NativeFile* File = nullptr;
		psd::Document* Document = nullptr;

		TSharedPtr<IPSDFileReader> FileReader = nullptr;

		File::FPSDDocument* Document2 = nullptr;

		TSharedPtr<FPSDFileImportVisitors> Visitors;

		FPSDFileImporterOptions Options;
	};

	/** For debugging purposes. */
	bool SaveImage(const FImage& InImage, const FString& InFilePath);

	/** Pads the channel so that it's placed correctly on the overall document canvas. */
	template <typename DataType>
	DataType* ExpandChannelToCanvas(psd::Allocator* InAllocator, const psd::Layer* InLayer, const void* InData, uint32 InWidth, uint32 InHeight)
	{
		DataType* Data = static_cast<DataType*>(InAllocator->Allocate(sizeof(DataType) * InWidth * InHeight, 16u));
		FMemory::Memset(Data, 0u, sizeof(DataType) * InWidth * InHeight);
		
		psd::imageUtil::CopyLayerData(static_cast<const DataType*>(InData), Data, InLayer->left, InLayer->top, InLayer->right, InLayer->bottom, InWidth, InHeight);

		return Data;
	}

	uint32 FindChannelIdx(const psd::Layer* InLayer, int16 InChannelType);

	FString GetLayerName(const psd::Layer* InLayer);
};
