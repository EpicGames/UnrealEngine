// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "Math/IntRect.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"

#include "PSDFileData.generated.h"

UENUM(BlueprintType)
enum class EPSDBlendMode : uint8
{
	PassThrough,
	Normal,
	Dissolve,

	Darken,
	Multiply,
	ColorBurn,
	LinearBurn,
	DarkerColor,

	Lighten,
	Screen,
	ColorDodge,
	LinearDodge,
	LighterColor,

	Overlay,
	SoftLight,
	HardLight,
	VividLight,
	LinearLight,
	PinLight,
	HardMix,

	Difference,
	Exclusion,
	Subtract,
	Divide,

	Hue,
	Saturation,
	Color,
	Luminosity,

	Unknown,
};

/** The types below map directly to structures within the PSD file format.  */
namespace UE::PSDImporter::File
{
	struct FPSDLayerRecord;

	enum class EPSDColorMode : int16
	{
		Bitmap = 0,
		Grayscale = 1,
		Indexed = 2,
		RGB = 3,
		CMYK = 4,
		Multichannel = 7,
		Duotone = 8,
		Lab = 9
	};

	enum class EPSDCompressionMethod : int16
	{
		Raw = 0,
		RLE = 1,
		ZIPWithoutPrediction = 2,
		ZIPWithPrediction = 3
	};

	enum class EPSDLayerType : int16
	{
		Any = 0,
		Group = 1,
	};

	/** @see: https://www.adobe.com/devnet-apps/photoshop/fileformatashtml/PSDFileFormats.htm#50577409_19840 */
	struct FPSDHeader
	{
		int32         Signature = 0;             // Must be 8BPS
		int16         Version = 0;               // Version: 1 = PSD, 2 = PSB
		uint8         Pad[6] = {};               // Padding
		int16         NumChannels = 0;           // Number of Channels: (3=RGB) (4=RGBA)
		int32         Height = 0;                // Number of Image Rows: 1-30,000 or 1-300,000 if PSB
		int32         Width = 0;                 // Number of Image Columns
		int16         Depth = 0;                 // Number of Bits per Channel: 1, 8, 16, 32
		EPSDColorMode Mode = EPSDColorMode::RGB; // Image Mode: (0=Bitmap)(1=Grayscale)(2=Indexed)(3=RGB)(4=CYMK)(7=Multichannel)

		bool IsValid() const;
	};

	struct FPSDImageData
	{
		EPSDCompressionMethod CompressionMethod;
	};

	enum class EPSDLayerFlags : uint8
	{
		None                  = 0,
		TransparencyProtected = 1 << 0,
		Visible               = 1 << 1,
		Obsolete              = 1 << 2,
		HasDataInBit4         = 1 << 3,
		NonVisiblePixelData   = 1 << 4		
	};
	ENUM_CLASS_FLAGS(EPSDLayerFlags);
	
	struct FPSDLayerAndMaskInformation
	{
		uint16 NumLayers = 0;
		bool bHasTransparencyMask = false;

		TSet<FPSDLayerRecord*> Layers;
	};

	PSDIMPORTERCORE_API const TCHAR* LexToString(EPSDColorMode InEnum);
}
