// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PSDFileData.h"

struct FPSDLayerRecord;

namespace UE::PSDImporter::File
{
	struct FPSDDocument
	{
		FPSDHeader Header;

		// Sections:
		// 1. ColorModeData (unsupported)
		// 2. ImageResources (unsupported)

		// 3. LayerAndMaskInformation
		FPSDLayerAndMaskInformation LayerAndMaskInformation;

		// 4. ImageData (unsupported)
	};
}
