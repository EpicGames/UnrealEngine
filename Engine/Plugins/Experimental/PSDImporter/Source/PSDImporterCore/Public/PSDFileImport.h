// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "ImageCore.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::PSDImporter
{
	namespace File
	{
		struct FPSDDocument;
		struct FPSDLayerAndMaskInformation;
		struct FPSDHeader;
		struct FPSDLayerRecord;
		struct FPSDLayerMaskData;
	}

	class IPSDFileReader;
	class FPSDFileImporter;

	struct FPSDFileImportVisitors
	{
		virtual ~FPSDFileImportVisitors() = default;

		virtual void OnImportComplete() { }
		
		using FHeaderInputType = UE::PSDImporter::File::FPSDHeader;
		virtual void OnImportHeader(const FHeaderInputType& InHeader) { }
		
		using FLayersInputType = UE::PSDImporter::File::FPSDLayerAndMaskInformation;
		virtual void OnImportLayers(const FLayersInputType& Layers) { }

		using FLayerInputType = UE::PSDImporter::File::FPSDLayerRecord;
		virtual void OnImportLayer(const FLayerInputType& InLayer, const FLayerInputType* InParentLayer, 
			TFunction<TFuture<FImage>()> InReadLayerData, TFunction<TFuture<FImage>()> InReadMaskData) { }
	};

	struct FPSDFileImporterOptions
	{
		bool bResizeLayersToDocument = false;
	};

	class FPSDFileImporter
	{
	public:
		PSDIMPORTERCORE_API static TSharedRef<FPSDFileImporter> Make(const FString& InFileName);

		virtual ~FPSDFileImporter() = default;

		virtual bool Import(const TSharedPtr<FPSDFileImportVisitors>& InVisitors, const FPSDFileImporterOptions& InOptions) = 0;

	protected:
		FPSDFileImporter() = default;
	};
}
