// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PSDFileData.h"

enum class EPSDBlendMode : uint8;

namespace UE::PSDImporter::File
{
	struct FPSDChannelInformation
	{
		int16 Id;     // 0 = Red, 1 = Green, 2 = Blue, -1 = Transparency Mask, -2 = User Mask, -3 = Both Masks
		int64 Length; // Data length
	};

	struct FPSDLayerRecord
	{
		int32 Index;
		FIntRect Bounds;
		uint16 NumChannels;
		EPSDBlendMode BlendMode;
		uint8 Opacity = 255;
		uint8 Clipping = 0;
		EPSDLayerFlags Flags;
		bool bIsGroup = false;
		FIntRect MaskBounds;
		uint8 MaskDefaultValue;

		FString LayerName;

		TSet<FPSDChannelInformation*> Channels;

		~FPSDLayerRecord();

		PSDIMPORTERCORE_API bool operator==(const FPSDLayerRecord& InOther) const;
		PSDIMPORTERCORE_API bool operator!=(const FPSDLayerRecord& InOther) const;
		PSDIMPORTERCORE_API bool operator<(const FPSDLayerRecord& InOther) const;
		PSDIMPORTERCORE_API bool operator<=(const FPSDLayerRecord& InOther) const;

		PSDIMPORTERCORE_API friend uint32 GetTypeHash(const FPSDLayerRecord& Value);
	};
}
