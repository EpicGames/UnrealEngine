// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PSDImporterEditor : ModuleRules
{
	public PSDImporterEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"PSDImporterCore",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"DeveloperSettings",
				"EditorFramework",
				"EditorSubsystem",
				"Engine",
				"ImageCore",
				"InputCore",
				"MaterialEditor",
				"PSDImporter",
				"Projects",
				"PsdSDK",
				"SequencerCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
			});
	}
}
