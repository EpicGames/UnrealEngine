// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PSDDocument.h"

#include "PSDDocument.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_PSDDocument"

FText UAssetDefinition_PSDDocument::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDefinitionDisplayName", "PSD Document");
}

FLinearColor UAssetDefinition_PSDDocument::GetAssetColor() const
{
	return FLinearColor::Blue;
}

TSoftClassPtr<UObject> UAssetDefinition_PSDDocument::GetAssetClass() const
{
	return UPSDDocument::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_PSDDocument::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Texture };
	return Categories;
}

bool UAssetDefinition_PSDDocument::CanImport() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
