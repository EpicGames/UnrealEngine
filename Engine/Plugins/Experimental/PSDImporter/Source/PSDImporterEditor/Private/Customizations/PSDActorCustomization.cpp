// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PSDActorCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "PSDActorCustomization"

namespace UE::PSDImporterEditor
{

TSharedRef<IDetailCustomization> FPSDActorCustomization::MakeInstance()
{
	return MakeShared<FPSDActorCustomization>();
}

void FPSDActorCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	InDetailBuilder.EditCategory(TEXT("PSD"), LOCTEXT("PSD", "PSD"))
		.SetSortOrder(2000);
}

} // UE::PSDImporterEditor

#undef LOCTEXT_NAMESPACE
