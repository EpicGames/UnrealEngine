// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

namespace UE::PSDImporterEditor
{

/** Sorts the PSD category to the top. */
class FPSDActorCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization
};

} // UE::PSDImporterEditor
