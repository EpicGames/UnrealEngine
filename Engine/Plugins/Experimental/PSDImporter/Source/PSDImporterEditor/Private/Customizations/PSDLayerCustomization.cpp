// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDLayerCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PSDFile.h"
#include "PSDImporterEditorLog.h"
#include "PSDImporterEditorUtilities.h"
#include "SlateMaterialBrush.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SToolTip.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PSDLayerCustomization"

namespace UE::PSDImporterEditor::Private
{
	constexpr const TCHAR* ThumbnailMaterialPath = TEXT("/PSDImporter/PSDImporter/M_PSDImporter_LayerPreview");
	constexpr int32 DefaultThumbnailSize = 48;

	UTexture2D* GetTextureFromHandle(const TSharedPtr<IPropertyHandle>& InHandle)
	{
		if (InHandle.IsValid())
		{
			UObject* Object = nullptr;

			if (InHandle->GetValue(Object) == FPropertyAccess::Success)
			{
				if (UTexture2D* Texture = Cast<UTexture2D>(Object))
				{
					if (Texture->GetSizeX() > 0)
					{
						return Texture;	
					}
				}
			}
		}

		return nullptr;
	};
}

TSharedRef<IPropertyTypeCustomization> UE::PSDImporterEditor::FPSDLayerCustomization::MakeInstance()
{
	return MakeShared<FPSDLayerCustomization>();
}

UE::PSDImporterEditor::FPSDLayerCustomization::FPSDLayerCustomization()
	: VisibleBrush(nullptr)
	, NotVisibleBrush(nullptr)
	, ThumbnailMaterialPath(UE::PSDImporterEditor::Private::ThumbnailMaterialPath)
{
	LayerThumbnailBrush = MakeShared<FSlateBrush>();
}

void UE::PSDImporterEditor::FPSDLayerCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, 
	FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	constexpr int32 MinThumbnailSize = 48;

	// Handles
	{
		LayerHandle = InPropertyHandle;

		FSimpleDelegate OnThumbnailChangedDelegate = FSimpleDelegate::CreateSP(this, &FPSDLayerCustomization::OnThumbnailChanged);
		ThumbnailHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, ThumbnailTexture));
		ThumbnailHandle->SetOnPropertyValueChanged(OnThumbnailChangedDelegate);
		ThumbnailHandle->SetOnChildPropertyValueChanged(OnThumbnailChangedDelegate);

		FSimpleDelegate OnTextureChangedDelegate = FSimpleDelegate::CreateSP(this, &FPSDLayerCustomization::OnLayerTextureChanged);
		LayerTextureHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, Texture));
		LayerTextureHandle->SetOnPropertyValueChanged(OnTextureChangedDelegate);
		LayerTextureHandle->SetOnChildPropertyValueChanged(OnTextureChangedDelegate);

		FSimpleDelegate OnMaskChangedDelegate = FSimpleDelegate::CreateSP(this, &FPSDLayerCustomization::OnMaskTextureChanged);
		MaskTextureHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, Mask));
		MaskTextureHandle->SetOnPropertyValueChanged(OnMaskChangedDelegate);
		MaskTextureHandle->SetOnChildPropertyValueChanged(OnMaskChangedDelegate);
	}

	FPSDFileLayer* Layer = GetLayer();

	if (!Layer)
	{
		return;
	}

	// Values
	const FIntRect Bounds = Layer->Bounds;
	const FIntRect MaskBounds = Layer->MaskBounds;
	const bool bHasMask = Layer->HasMask();

	TSharedPtr<IPropertyHandle> ImportOperationHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, ImportOperation));
	TSharedPtr<IPropertyHandle> VisibilityHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, bIsVisible));
	TSharedPtr<IPropertyHandle> NameHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, Id))->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayerId, Name));
	TSharedPtr<IPropertyHandle> BlendModeHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, BlendMode));
	TSharedPtr<IPropertyHandle> OpacityHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, Opacity));
	TSharedPtr<IPropertyHandle> ClippingHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPSDFileLayer, Clipping));

	// Style
	FMargin Padding(4, 0);
	double IndentWidth = 0.0;

	{
		static const FName NAME_VisibleBrush = TEXT("Level.VisibleIcon16x");
		static const FName NAME_NotVisibleBrush = TEXT("Level.NotVisibleIcon16x");

		VisibleBrush = FAppStyle::Get().GetBrush(NAME_VisibleBrush);
		NotVisibleBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleBrush);

		double IndentSpacing = 20.0;
		int32 IndentLevel = 0;

		if (Layer && Layer->ParentId.IsSet())
		{
			++IndentLevel;
		}

		IndentWidth = IndentSpacing * IndentLevel;

		// Thumbnail Brushes
		{
			if (UMaterialInterface* ThumbnailMaterial = Cast<UMaterialInterface>(ThumbnailMaterialPath.TryLoad()))
			{
				LayerThumbnailMID = UMaterialInstanceDynamic::Create(ThumbnailMaterial, GetTransientPackage());
				LayerThumbnailBrush = MakeShared<FSlateMaterialBrush>(*LayerThumbnailMID, FVector2D(MinThumbnailSize, MinThumbnailSize));

				LayerToolTipThumbnailMID = UMaterialInstanceDynamic::Create(ThumbnailMaterial, GetTransientPackage());
				LayerToolTipThumbnailBrush = MakeShared<FSlateMaterialBrush>(*LayerToolTipThumbnailMID, FVector2D(MinThumbnailSize, MinThumbnailSize));

				if (bHasMask)
				{
					MaskThumbnailMID = UMaterialInstanceDynamic::Create(ThumbnailMaterial, GetTransientPackage());
					MaskThumbnailBrush = MakeShared<FSlateMaterialBrush>(*MaskThumbnailMID, FVector2D(MinThumbnailSize, MinThumbnailSize));

					MaskToolTipThumbnailMID = UMaterialInstanceDynamic::Create(ThumbnailMaterial, GetTransientPackage());
					MaskToolTipThumbnailBrush = MakeShared<FSlateMaterialBrush>(*MaskToolTipThumbnailMID, FVector2D(MinThumbnailSize, MinThumbnailSize));
				}
			}
			else
			{
				UE_LOGF(LogPSDImporterEditor, Warning, "ThumbnailMaterial could not be loaded from path: '%ls'", *ThumbnailMaterialPath.ToString());
			}
		}

		// Update Thumbnail
		if (UTexture2D* ThumbnailTexture = Private::GetTextureFromHandle(ThumbnailHandle))
		{
			UpdateLayerThumbnail(ThumbnailTexture);
		}

		if (bHasMask)
		{
			if (UTexture2D* MaskTexture = Private::GetTextureFromHandle(MaskTextureHandle))
			{
				UpdateMaskThumbnail(MaskTexture);
			}
		}

		// Update ToolTip Thumbnail
		if (Layer)
		{
			UpdateLayerToolTipThumbnail(Layer->Texture.LoadSynchronous());

			if (bHasMask)
			{
				UpdateMaskToolTipThumbnail(Layer->Mask.LoadSynchronous());
			}
		}
	}

	TSharedPtr<SToolTip> LayerToolTipWidget = SNew(SToolTip)
		.BorderImage(FAppStyle::Get().GetBrush("ToolTip.Background"))
		.TextMargin(2.f)
		[
			SNew(SImage)
			.Visibility(this, &FPSDLayerCustomization::GetToolTipLayerThumbnailVisibility)
			.Image(this, &FPSDLayerCustomization::GetToolTipLayerThumbnailBrush)
		];

	TSharedPtr<SToolTip> MaskToolTipWidget;

	if (bHasMask)
	{
		MaskToolTipWidget = SNew(SToolTip)
			.BorderImage(FAppStyle::Get().GetBrush("ToolTip.Background"))
			.TextMargin(2.f)
			[
				SNew(SImage)
				.Visibility(this, &FPSDLayerCustomization::GetToolTipMaskThumbnailVisibility)
				.Image(this, &FPSDLayerCustomization::GetToolTipMaskThumbnailBrush)
			];
	}

	InHeaderRow.WholeRowContent()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)			

			+ SHorizontalBox::Slot()
			.Padding(Padding)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.IsEnabled(false)
				.Image(this, &FPSDLayerCustomization::GetVisibilityBrush)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpacer)
				.Size(FVector2D(IndentWidth, 0))
			]

			+ SHorizontalBox::Slot()
			.Padding(Padding)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(FOnClicked::CreateLambda([WeakThis = SharedThis(this).ToWeakPtr()]
				{
					if (TSharedPtr<FPSDLayerCustomization> StrongThis = WeakThis.Pin())
					{
						if (const FPSDFileLayer* InLayer = StrongThis->GetLayer())
						{
							Private::SelectLayerTextureAsset(*InLayer);
							return FReply::Handled();
						}
					}

					return FReply::Unhandled();
				}))
				.Content()
				[
					SNew(SBox)
					.WidthOverride(50.f)
					.HeightOverride(50.f)
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(SImage)
						.ToolTip(LayerToolTipWidget)
						.Image(this, &FPSDLayerCustomization::GetLayerThumbnailBrush)
						.ColorAndOpacity(TAttribute<FSlateColor>::CreateLambda([WeakThis = SharedThis(this).ToWeakPtr(), GetTextureFromHandle = Private::GetTextureFromHandle]()
						{
							FLinearColor Result = FLinearColor::White;
							Result.A = 0.0;

							if (TSharedPtr<FPSDLayerCustomization> StrongThis = WeakThis.Pin())
							{
								if (UTexture2D* ThumbnailTexture = GetTextureFromHandle(StrongThis->ThumbnailHandle))
								{
									Result.A = ThumbnailTexture ? 1.0 : 0.0;
								}
							}

							return Result;
						}))
					]
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(Padding)
			.AutoWidth()
			[
				SNew(SButton)
				.Visibility(bHasMask ? EVisibility::Visible : EVisibility::Hidden)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(FOnClicked::CreateLambda([WeakThis = SharedThis(this).ToWeakPtr()]
				{
					if (TSharedPtr<FPSDLayerCustomization> StrongThis = WeakThis.Pin())
					{
						if (const FPSDFileLayer* InLayer = StrongThis->GetLayer())
						{
							Private::SelectMaskTextureAsset(*InLayer);
							return FReply::Handled();
						}
					}

					return FReply::Unhandled();
				}))
				.Content()
				[
					SNew(SBox)
					.WidthOverride(50.f)
					.HeightOverride(50.f)
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(SImage)
						.ToolTip(MaskToolTipWidget)
						.Image(this, &FPSDLayerCustomization::GetMaskThumbnailBrush)
						.ColorAndOpacity(TAttribute<FSlateColor>::CreateLambda([WeakThis = SharedThis(this).ToWeakPtr(), GetTextureFromHandle = Private::GetTextureFromHandle]()
						{
							FLinearColor Result = FLinearColor::White;
							Result.A = 0.0;

							if (TSharedPtr<FPSDLayerCustomization> StrongThis = WeakThis.Pin())
							{
								if (UTexture2D* ThumbnailTexture = GetTextureFromHandle(StrongThis->ThumbnailHandle))
								{
									Result.A = ThumbnailTexture ? 1.0 : 0.0;
								}
							}

							return Result;
						}))
					]
				]
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(240 - IndentWidth)
			.Padding(Padding)
			[
				NameHandle->CreatePropertyValueWidget()
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(120)
			.Padding(Padding)
			[
				BlendModeHandle->CreatePropertyValueWidget()
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(60)
			.Padding(Padding)
			[
				OpacityHandle->CreatePropertyValueWidget()
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(60)
			.Padding(Padding)
			[
				ClippingHandle->CreatePropertyValueWidget()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(Padding)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SVerticalBox)+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(INVTEXT("-"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(LOCTEXT("Base", "Base:"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					bHasMask
						? SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFontBold())
							.Text(LOCTEXT("Mask", "Mask:"))
						: SNullWidget::NullWidget
				]
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(75.f)
			.Padding(Padding)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SVerticalBox)				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(LOCTEXT("Position", "Position"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::Format(
						LOCTEXT("PositionFormat", "{0}, {1}"),
						FText::AsNumber(Bounds.Min.X),
						FText::AsNumber(Bounds.Min.Y)
					))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					bHasMask
						? SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(FText::Format(
								LOCTEXT("PositionFormat", "{0}, {1}"),
								FText::AsNumber(MaskBounds.Min.X),
								FText::AsNumber(MaskBounds.Min.Y)
							))
						: SNullWidget::NullWidget
				]
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(75.f)
			.Padding(Padding)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(LOCTEXT("Size", "Size"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::Format(
						LOCTEXT("SizeFormat", "{0} x {1}"),
						FText::AsNumber(Bounds.Max.X - Bounds.Min.X),
						FText::AsNumber(Bounds.Max.Y - Bounds.Min.Y)
					))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					bHasMask
						? SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(FText::Format(
								LOCTEXT("SizeFormat", "{0} x {1}"),
								FText::AsNumber(MaskBounds.Max.X - MaskBounds.Min.X),
								FText::AsNumber(MaskBounds.Max.Y - MaskBounds.Min.Y)
							))
						: SNullWidget::NullWidget
				]
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(75.f)
			.Padding(Padding)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(LOCTEXT("Default", "Default:"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(INVTEXT("-"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					bHasMask
						? SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(FText::AsNumber(Layer->MaskDefaultValue))
						: SNullWidget::NullWidget
				]
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.MaxWidth(240)
			[
				SNew(SProgressBar)
				.Visibility(TAttribute<EVisibility>::CreateLambda([WeakThis = SharedThis(this).ToWeakPtr()]()
				{
					if (TSharedPtr<FPSDLayerCustomization> StrongThis = WeakThis.Pin())
					{
						return StrongThis->IsLoading()
							? EVisibility::Visible
							: EVisibility::Collapsed;
					}
					return EVisibility::Collapsed;
				}))
				.Percent(0.2)
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(120)
			[
				ImportOperationHandle->CreatePropertyValueWidget()
			]
		]
	];
}

void UE::PSDImporterEditor::FPSDLayerCustomization::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(LayerThumbnailMID);
	InCollector.AddReferencedObject(LayerToolTipThumbnailMID);
	InCollector.AddReferencedObject(MaskThumbnailMID);
	InCollector.AddReferencedObject(MaskToolTipThumbnailMID);
}

FString UE::PSDImporterEditor::FPSDLayerCustomization::GetReferencerName() const
{
	return TEXT("FPSDLayerCustomization");
}

void UE::PSDImporterEditor::FPSDLayerCustomization::OnThumbnailChanged()
{
	if (UTexture2D* Texture = Private::GetTextureFromHandle(ThumbnailHandle))
	{
		UpdateLayerThumbnail(Texture);
	}
}

void UE::PSDImporterEditor::FPSDLayerCustomization::OnLayerTextureChanged()
{
	if (UTexture2D* Texture = Private::GetTextureFromHandle(LayerTextureHandle))
	{
		UpdateLayerToolTipThumbnail(Texture);
	}
}

void UE::PSDImporterEditor::FPSDLayerCustomization::OnMaskTextureChanged()
{
	if (UTexture2D* Texture = Private::GetTextureFromHandle(MaskTextureHandle))
	{
		UpdateLayerToolTipThumbnail(Texture);
	}
}

FPSDFileLayer* UE::PSDImporterEditor::FPSDLayerCustomization::GetLayer() const
{
	if (LayerHandle.IsValid())
	{
		void* Data;

		if (LayerHandle->GetValueData(Data) == FPropertyAccess::Success)
		{
			return static_cast<FPSDFileLayer*>(Data);
		}
	}
	
	return nullptr;
}

const FSlateBrush* UE::PSDImporterEditor::FPSDLayerCustomization::GetVisibilityBrush() const
{
	if (const FPSDFileLayer* Layer = GetLayer())
	{
		return Layer->bIsVisible ? VisibleBrush : NotVisibleBrush;
	}
	
	return VisibleBrush;
}

const FSlateBrush* UE::PSDImporterEditor::FPSDLayerCustomization::GetLayerThumbnailBrush() const
{
	if (LayerThumbnailBrush.IsValid())
	{
		return LayerThumbnailBrush.Get();
	}

	return nullptr;
}

EVisibility UE::PSDImporterEditor::FPSDLayerCustomization::GetToolTipLayerThumbnailVisibility() const
{
	return LayerToolTipThumbnailBrush.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* UE::PSDImporterEditor::FPSDLayerCustomization::GetToolTipLayerThumbnailBrush() const
{
	if (LayerToolTipThumbnailBrush.IsValid())
	{
		return LayerToolTipThumbnailBrush.Get();
	}

	return nullptr;
}

const FSlateBrush* UE::PSDImporterEditor::FPSDLayerCustomization::GetMaskThumbnailBrush() const
{
	if (MaskThumbnailBrush.IsValid())
	{
		return MaskThumbnailBrush.Get();
	}

	return nullptr;
}

EVisibility UE::PSDImporterEditor::FPSDLayerCustomization::GetToolTipMaskThumbnailVisibility() const
{
	return MaskToolTipThumbnailBrush.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* UE::PSDImporterEditor::FPSDLayerCustomization::GetToolTipMaskThumbnailBrush() const
{
	if (MaskToolTipThumbnailBrush.IsValid())
	{
		return MaskToolTipThumbnailBrush.Get();
	}

	return nullptr;
}

void UE::PSDImporterEditor::FPSDLayerCustomization::UpdateLayerThumbnail(UTexture2D* InTexture)
{
	UpdateThumbnailInternal(InTexture, LayerThumbnailMID, LayerThumbnailBrush);
}

void UE::PSDImporterEditor::FPSDLayerCustomization::UpdateLayerToolTipThumbnail(UTexture2D* InTexture)
{
	constexpr int32 ToolTipThumbnailSize = 256;
	UpdateThumbnailInternal(InTexture, LayerToolTipThumbnailMID, LayerToolTipThumbnailBrush, ToolTipThumbnailSize);
}

void UE::PSDImporterEditor::FPSDLayerCustomization::UpdateMaskThumbnail(UTexture2D* InTexture)
{
	UpdateThumbnailInternal(InTexture, MaskThumbnailMID, MaskThumbnailBrush);
}

void UE::PSDImporterEditor::FPSDLayerCustomization::UpdateMaskToolTipThumbnail(UTexture2D* InTexture)
{
	constexpr int32 ToolTipThumbnailSize = 256;
	UpdateThumbnailInternal(InTexture, MaskToolTipThumbnailMID, MaskToolTipThumbnailBrush, ToolTipThumbnailSize);
}

void UE::PSDImporterEditor::FPSDLayerCustomization::UpdateThumbnailInternal(UTexture2D* InTexture, UMaterialInstanceDynamic* InMID, const TSharedPtr<FSlateBrush>& InBrush, TOptional<int32> InMaxSize)
{
	if (!ensure(InBrush.IsValid()))
	{
		return;
	}
	
	FVector2D ThumbnailSize(Private::DefaultThumbnailSize, Private::DefaultThumbnailSize);
	if (!InTexture)
	{
		InTexture = nullptr;
	}
	else
	{
		ThumbnailSize = FVector2D(InTexture->GetImportedSize().X, InTexture->GetImportedSize().Y);
	}

	if (InMaxSize.IsSet())
	{
		const int32 MaxSize = InMaxSize.GetValue();
		ThumbnailSize = Private::FitMinClampMaxXY(ThumbnailSize, MaxSize, MaxSize);
	}

	InBrush->ImageSize = ThumbnailSize;
	
	if (InMID != nullptr)
	{
		InMID->SetTextureParameterValue(TEXT("Texture"), InTexture);
	}
}

bool UE::PSDImporterEditor::FPSDLayerCustomization::IsLoading() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
