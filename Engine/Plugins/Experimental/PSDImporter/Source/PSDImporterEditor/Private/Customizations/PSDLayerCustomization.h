// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "UObject/GCObject.h"

#include "Layout/Visibility.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"

class UTexture2D;
struct FPSDFileLayer;
struct FSlateBrush;

namespace UE::PSDImporterEditor
{
	class FPSDLayerCustomization
		: public IPropertyTypeCustomization
		, public FGCObject // To handle Texture referencing/GC
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		FPSDLayerCustomization();

		//~ Begin IPropertyTypeCustomization
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, 
			IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, 
			IPropertyTypeCustomizationUtils& InCustomizationUtils) override {}
		//~ End IPropertyTypeCustomization

		// ~Begin FGCObject Interface
		virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
		virtual FString GetReferencerName() const override;
		// ~End FGCObject Interface

	private:
		void OnThumbnailChanged();
		void OnLayerTextureChanged();
		void OnMaskTextureChanged();

		FPSDFileLayer* GetLayer() const;

		const FSlateBrush* GetVisibilityBrush() const;
		const FSlateBrush* GetLayerThumbnailBrush() const;
		EVisibility GetToolTipLayerThumbnailVisibility() const;
		const FSlateBrush* GetToolTipLayerThumbnailBrush() const;
		const FSlateBrush* GetMaskThumbnailBrush() const;
		EVisibility GetToolTipMaskThumbnailVisibility() const;
		const FSlateBrush* GetToolTipMaskThumbnailBrush() const;
		
		void UpdateLayerThumbnail(UTexture2D* InTexture);
		void UpdateLayerToolTipThumbnail(UTexture2D* InTexture);
		void UpdateMaskThumbnail(UTexture2D* InTexture);
		void UpdateMaskToolTipThumbnail(UTexture2D* InTexture);
		void UpdateThumbnailInternal(UTexture2D* InTexture, UMaterialInstanceDynamic* InMID, const TSharedPtr<FSlateBrush>& InBrush, TOptional<int32> InMaxSize = {});
		
		bool IsLoading() const;

	private:
		const FSlateBrush* VisibleBrush;
		const FSlateBrush* NotVisibleBrush;
		
		FSoftObjectPath ThumbnailMaterialPath;
		
		TObjectPtr<UMaterialInstanceDynamic> LayerThumbnailMID;
		TSharedPtr<FSlateBrush> LayerThumbnailBrush;

		TObjectPtr<UMaterialInstanceDynamic> LayerToolTipThumbnailMID;
		TSharedPtr<FSlateBrush> LayerToolTipThumbnailBrush;

		TObjectPtr<UMaterialInstanceDynamic> MaskThumbnailMID;
		TSharedPtr<FSlateBrush> MaskThumbnailBrush;

		TObjectPtr<UMaterialInstanceDynamic> MaskToolTipThumbnailMID;
		TSharedPtr<FSlateBrush> MaskToolTipThumbnailBrush;

		TSharedPtr<IPropertyHandle> LayerHandle;
		TSharedPtr<IPropertyHandle> ThumbnailHandle;
		TSharedPtr<IPropertyHandle> LayerTextureHandle;
		TSharedPtr<IPropertyHandle> MaskTextureHandle;
	};
}
