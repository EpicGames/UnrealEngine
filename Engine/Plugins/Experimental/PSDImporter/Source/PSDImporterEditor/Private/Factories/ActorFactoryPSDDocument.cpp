// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ActorFactoryPSDDocument.h"

#include "PSDDocument.h"
#include "PSDImporterEditorUtilities.h"
#include "PSDQuadActor.h"
#include "PSDQuadsFactory.h"

#define LOCTEXT_NAMESPACE "ActorFactoryPSDDocument"

UActorFactoryPSDDocument::UActorFactoryPSDDocument()
{
	DisplayName = LOCTEXT("PSDDocumentDisplayName", "PSD Document");
	NewActorClass = UPSDDocument::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UActorFactoryPSDDocument::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UPSDDocument::StaticClass()))
	{
		OutErrorMsg = LOCTEXT("NotPSDDocument", "A valid photoshop document was not selected.");
		return false;
	}

	return true;
}

UClass* UActorFactoryPSDDocument::GetDefaultActorClass(const FAssetData& AssetData)
{
	return APSDQuadActor::StaticClass();
}

void UActorFactoryPSDDocument::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	using namespace UE::PSDImporterEditor::Private;

	Super::PostSpawnActor(Asset, NewActor);

	if (NewActor->bIsEditorPreviewActor)
	{
		NewActor->SetActorEnableCollision(false);
	}

	APSDQuadActor* QuadActor = Cast<APSDQuadActor>(NewActor);

	if (!QuadActor)
	{
		return;
	}

	QuadActor->SetActorScale3D(FVector(InitialScale, InitialScale, InitialScale));

	if (!QuadActor->GetPSDDocument())
	{
		if (UPSDDocument* PSDDocument = Cast<UPSDDocument>(Asset))
		{
			QuadActor->SetPSDDocument(*PSDDocument);
			GetMutableDefault<UPSDQuadsFactory>()->CreateQuads(*QuadActor);
		}
	}
}

UObject* UActorFactoryPSDDocument::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	APSDQuadActor* QuadActor = CastChecked<APSDQuadActor>(Instance);

	check(QuadActor->GetPSDDocument());
	return QuadActor->GetPSDDocument();
}

FQuat UActorFactoryPSDDocument::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
	// Meshes align the Z (up) axis with the surface normal
	return FindActorAlignmentRotation(ActorRotation, FVector(0.f, 0.f, 1.f), InSurfaceNormal);
}

#undef LOCTEXT_NAMESPACE
