// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"

#include "UObject/ObjectMacros.h"

#include "ActorFactoryPSDDocument.generated.h"

UCLASS()
class UActorFactoryPSDDocument : public UActorFactory
{
	GENERATED_BODY()

public:
	UActorFactoryPSDDocument();

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual UClass* GetDefaultActorClass(const FAssetData& AssetData) override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	virtual FQuat AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const override;
	//~ End UActorFactory Interface
};
