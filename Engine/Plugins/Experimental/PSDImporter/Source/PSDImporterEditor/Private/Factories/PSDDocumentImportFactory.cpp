// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDDocumentImportFactory.h"

#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "Factories/PSDDocumentImportFactory_Visitors.h"
#include "Misc/ScopedSlowTask.h"
#include "PSDDocument.h"
#include "PSDFileImport.h"
#include "PSDImporterEditorSettings.h"
#include "Subsystems/ImportSubsystem.h"

#define LOCTEXT_NAMESPACE "PSDDocumentImportFactory"

UPSDDocumentImportFactory::UPSDDocumentImportFactory()
{
	SupportedClass = UPSDDocument::StaticClass();

	Formats.Add(TEXT("psd;PSD"));
	// Formats.Add(TEXT("psb;PSD")); // unsupported by lib

	bCreateNew = bText = false;
	bEditAfterNew = true;
	bEditorImport = true;
	ImportPriority += 100;
}

UObject* UPSDDocumentImportFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, 
	const FString& InFilename, const TCHAR* InParams, FFeedbackContext* InWarn, bool& bInOutOperationCanceled)
{
	UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>();
	check(ImportSubsystem);

	if (!ensureAlways(InClass == UPSDDocument::StaticClass()))
	{
		return nullptr;
	}

	UPSDDocument* NewObjectAsset = NewObject<UPSDDocument>(InParent, InClass, InName, InFlags | RF_Transactional);
	NewObjectAsset->AssetImportData->AddFileName(InFilename, 0);

	if (UPSDImporterEditorSettings* Settings = UPSDImporterEditorSettings::Get())
	{
		NewObjectAsset->bImportInvisibleLayers = Settings->bImportInvisibleLayers;
		NewObjectAsset->bResizeLayersToDocument = Settings->bResizeLayersToDocument;
		NewObjectAsset->bLayersResizedOnImport = NewObjectAsset->bResizeLayersToDocument;
	}

	if (Import(InFilename, NewObjectAsset))
	{
		return NewObjectAsset;
	}

	return nullptr;
}

bool UPSDDocumentImportFactory::CanReimport(UObject* InObj, TArray<FString>& OutFilenames)
{
	if (const UPSDDocument* Document = Cast<UPSDDocument>(InObj))
	{
		if (Document->AssetImportData)
		{
			Document->AssetImportData->ExtractFilenames(OutFilenames);
			const FString Filename = Document->AssetImportData->GetFirstFilename();
			return FPaths::GetExtension(Filename) == TEXT("psd");
		}
	}

	return true;
}

void UPSDDocumentImportFactory::SetReimportPaths(UObject* InObj, const TArray<FString>& InNewReimportPaths)
{
	if (InNewReimportPaths.IsEmpty())
	{
		return;
	}

	if (const UPSDDocument* Document = Cast<UPSDDocument>(InObj))
	{
		if (FactoryCanImport(InNewReimportPaths[0]))
		{
			Document->AssetImportData->UpdateFilenameOnly(InNewReimportPaths[0]);
		}
	}
}

EReimportResult::Type UPSDDocumentImportFactory::Reimport(UObject* InObj)
{
	if (UPSDDocument* Document = Cast<UPSDDocument>(InObj))
	{
		const FString FilePath = Document->AssetImportData->GetFirstFilename();
		Document->bLayersResizedOnImport = Document->bResizeLayersToDocument;

		if (Import(FilePath, Document))
		{
			return EReimportResult::Succeeded; 
		}
	}

	return EReimportResult::Failed;
}

bool UPSDDocumentImportFactory::Import(const FString& InFilePath, UPSDDocument* InDocument)
{
	const TSharedPtr<FPSDDocumentImportFactory_Visitors> Visitors = MakeShared<FPSDDocumentImportFactory_Visitors>(InFilePath, InDocument);
	const TSharedRef<UE::PSDImporter::FPSDFileImporter> Importer = UE::PSDImporter::FPSDFileImporter::Make(InFilePath);

	UE::PSDImporter::FPSDFileImporterOptions Options;
	Options.bResizeLayersToDocument = InDocument->bResizeLayersToDocument;

	FScopedSlowTask SlowTask(1.f, LOCTEXT("ImportingPSDFile", "Importing PSD file..."));
	SlowTask.MakeDialog();
	SlowTask.EnterProgressFrame(1.f);

	return Importer->Import(Visitors, Options);
}

#undef LOCTEXT_NAMESPACE
