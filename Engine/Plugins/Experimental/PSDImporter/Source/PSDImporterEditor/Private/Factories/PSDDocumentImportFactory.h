// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "Factories/Factory.h"

#include "PSDDocumentImportFactory.generated.h"

class UPSDDocument;

UCLASS()
class UPSDDocumentImportFactory
	: public UFactory
	, public FReimportHandler
{
	GENERATED_BODY()

public:
	UPSDDocumentImportFactory();

	//~ Begin UFactory
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& InFilename, 
		const TCHAR* InParams, FFeedbackContext* InWarn, bool& bInOutOperationCanceled) override;
	//~ End UFactory

	//~ Begin FReimportHandler
	virtual bool CanReimport(UObject* InObj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* InObj, const TArray<FString>& InNewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* InObj) override;
	//~ End FReimportHandler

private:
	bool Import(const FString& InFilePath, UPSDDocument* InDocument);
};
