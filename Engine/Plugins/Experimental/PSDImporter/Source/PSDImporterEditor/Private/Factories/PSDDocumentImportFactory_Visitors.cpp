// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/PSDDocumentImportFactory_Visitors.h"

#include "AssetToolsModule.h"
#include "Containers/Set.h"
#include "Editor.h"
#include "IAssetTools.h"
#include "ObjectTools.h"
#include "PSDDocument.h"
#include "PSDFile.h"
#include "PSDFileData.h"
#include "PSDFileRecord.h"
#include "PSDImporterEditorLog.h"
#include "PSDImporterEditorUtilities.h"
#include "PSDLayerTextureUserData.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"

namespace UE::PSDImporter::Private
{
	void Sanitize(FString& OutPath, const FString& InInvalidChars, const FString::ElementType InReplaceWith)
	{
		// Make sure the package path only contains valid characters
		FText OutError;
		
		if (FName::IsValidXName(OutPath, InInvalidChars, &OutError))
		{
			return;
		}

		for (const FString::ElementType& InvalidChar : InInvalidChars)
		{
			OutPath.ReplaceCharInline(InvalidChar, InReplaceWith, ESearchCase::CaseSensitive);
		}
	}

	// Replaces any invalid package path characters 
	void SanitizePackagePath(FString& OutPath, FString::ElementType InReplaceWith = TEXT('_'))
	{
		FPaths::RemoveDuplicateSlashes(OutPath);
		return Private::Sanitize(OutPath, INVALID_OBJECTPATH_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, InReplaceWith);
	}

	// Replaces any invalid object name characters 
	void SanitizeAssetName(FString& OutPath, FString::ElementType InReplaceWith = TEXT('_'))
	{
		return Private::Sanitize(OutPath, INVALID_OBJECTNAME_CHARACTERS, InReplaceWith);
	}
}

FPSDDocumentImportFactory_Visitors::FPSDDocumentImportFactory_Visitors(const FString& InFilePath, UPSDDocument* InDocument)
	: FilePath(InFilePath)
    , Document(InDocument)
    , FileDocument(InDocument->FileDocument)
{
	OldLayers = TSet<FPSDFileLayer>(InDocument->Layers);
	NewLayers.Reserve(InDocument->FileDocument.Layers.Num());
}

void FPSDDocumentImportFactory_Visitors::OnImportComplete()
{
	FString Unused;
	FPaths::Split(FilePath, Unused, Document->DocumentName, Unused);

	UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();
	check(EditorAssetSubsystem);

	auto DeleteLayerTexture = [this, EditorAssetSubsystem](const FPSDFileLayer& InLayer)
	{
		FString LayerTexturePath;
		FString LayerTextureName;
		MakeAssetPath(
			Document->DocumentName,
			FString::Printf(TEXT("%s_%i"), *InLayer.Id.Name, InLayer.Id.Index),
			TEXT("T"),
			LayerTexturePath,
			LayerTextureName);

		if (EditorAssetSubsystem->DoesAssetExist(LayerTexturePath))
		{
			LayerTexturePath = FPaths::Combine(LayerTexturePath, LayerTextureName);
		}

		if (!EditorAssetSubsystem->DeleteAsset(LayerTexturePath))
		{
			UE_LOGF(LogPSDImporterEditor, Error, "Error deleting texture asset for layer.");
		}
	};

	// Cleanup old layers
	OldLayers = OldLayers.Difference(NewLayers);
	for (FPSDFileLayer& OldLayer : OldLayers)
	{
		DeleteLayerTexture(OldLayer);
	}

	// Delete "ignored" layer textures
	for (FPSDFileLayer& NewLayer : NewLayers)
	{
		if (NewLayer.ImportOperation == EPSDFileLayerImportOperation::Ignore)
		{
			DeleteLayerTexture(NewLayer);
		}
	}

	FileDocument.Layers = NewLayers;

	Document->Size = FIntPoint(FileDocument.Width, FileDocument.Height);

	Document->Layers = FileDocument.Layers.Array();
	Document->Layers.Sort(); // Sorts by layer index
}

void FPSDDocumentImportFactory_Visitors::OnImportHeader(const FHeaderInputType& InHeader)
{
	FileDocument.Width = InHeader.Width;
	FileDocument.Height = InHeader.Height;
	FileDocument.Depth = InHeader.Depth;
	FileDocument.ColorMode = FName(LexToString(InHeader.Mode));
}

void FPSDDocumentImportFactory_Visitors::OnImportLayers(const FLayersInputType& InLayers)
{
	FileDocument.Layers.Reserve(InLayers.NumLayers);
}

void FPSDDocumentImportFactory_Visitors::OnImportLayer(const FLayerInputType& InLayer, const FLayerInputType* InParentLayer,
	TFunction<TFuture<FImage>()> InReadLayerData, TFunction<TFuture<FImage>()> InReadMaskData)
{
	FPSDFileLayer* Layer = OldLayers.Find(FPSDFileLayer(InLayer.Index, InLayer.LayerName, InLayer.bIsGroup ? EPSDFileLayerType::Group : EPSDFileLayerType::Any));
	if (!Layer)
	{
		Layer = new FPSDFileLayer();
	}

	Layer->Bounds = InLayer.Bounds;
	Layer->Id = FPSDFileLayerId(InLayer.Index, InLayer.LayerName);
	Layer->Opacity = static_cast<double>(InLayer.Opacity) / 255;
	Layer->Type = InLayer.bIsGroup ? EPSDFileLayerType::Group : EPSDFileLayerType::Any;
	Layer->bIsVisible = EnumHasAnyFlags(InLayer.Flags, UE::PSDImporter::File::EPSDLayerFlags::Visible);
	Layer->BlendMode = InLayer.BlendMode;
	Layer->MaskBounds = InLayer.MaskBounds;
	Layer->MaskDefaultValue = InLayer.MaskDefaultValue;
	Layer->Clipping = InLayer.Clipping;

	if (InParentLayer)
	{
		Layer->ParentId = FPSDFileLayerId(InParentLayer->Index, InParentLayer->LayerName);
	}

	if (InReadLayerData)
	{
		TFuture<FImage> LayerImageTask = InReadLayerData();
		LayerImageTask.Wait();

		const FImage& InLayerImage = LayerImageTask.Get();

		if (InLayerImage.IsImageInfoValid()
			&& !InLayerImage.RawData.IsEmpty())
		{
			// Texture Asset
			{
				TFunction<bool(const FPSDFileLayer* InLayerToCheck)> HasMergedParent;
				HasMergedParent = [&](const FPSDFileLayer* InLayerToCheck) -> bool
				{
					if (!InLayerToCheck->ParentId.IsSet())
					{
						return false;
					}

					if (const FPSDFileLayer* ParentLayer = NewLayers.Find(InLayerToCheck->ParentId.GetValue()))
					{
						if (ParentLayer->ImportOperation == EPSDFileLayerImportOperation::Ignore)
						{
							return false;
						}

						if (ParentLayer->ImportOperation == EPSDFileLayerImportOperation::ImportMerged
							&& (ParentLayer->bIsVisible || Document->bImportInvisibleLayers))
						{
							return true;
						}

						return HasMergedParent(ParentLayer);
					}

					return false;
				};

				if (Layer->ImportOperation != EPSDFileLayerImportOperation::Ignore
					&& (Layer->bIsVisible || Document->bImportInvisibleLayers)
					&& !HasMergedParent(Layer))
				{
					auto MakeValid = [](double& Value)
						{
							Value = FMath::IsNaN(Value) ? 0.0 : Value;
						};

					UTexture2D* LayerTexture = Layer->Texture.LoadSynchronous();
					if (!LayerTexture || !LayerTexture->IsAsset())
					{
						LayerTexture = Cast<UTexture2D>(
							MakeAsset(UTexture2D::StaticClass(),
							          Document->GetName(),
							          FString::Printf(TEXT("%s_%i"), *Layer->Id.Name, Layer->Id.Index),
							          TEXT("T")));
					}

					if (!LayerTexture)
					{
						UE_LOGF(LogPSDImporterEditor, Error, "Error creating texture for layer");
					}
					else
					{
						Layer->Texture = LayerTexture;

						LayerTexture->PreEditChange(nullptr);
						LayerTexture->Source.Init(InLayerImage);
						LayerTexture->Source.Compress();

						UPSDLayerTextureUserData* LayerTextureUserData = LayerTexture->GetAssetUserData<UPSDLayerTextureUserData>();
						if (!LayerTextureUserData)
						{
							LayerTextureUserData = NewObject<UPSDLayerTextureUserData>(LayerTexture, NAME_None, RF_Public | RF_Transactional);
							LayerTexture->AddAssetUserData(LayerTextureUserData);
						}

						LayerTextureUserData->LayerId = Layer->Id;

						LayerTextureUserData->NormalizedBounds = FBox2D(FVector2d(Layer->Bounds.Min) / Document->Size, FVector2d(Layer->Bounds.Max) / Document->Size);

						MakeValid(LayerTextureUserData->NormalizedBounds.Min.X);
						MakeValid(LayerTextureUserData->NormalizedBounds.Min.Y);
						MakeValid(LayerTextureUserData->NormalizedBounds.Max.X);
						MakeValid(LayerTextureUserData->NormalizedBounds.Max.Y);

						LayerTextureUserData->PixelBounds = Layer->Bounds;

						LayerTexture->PostEditChange();
					}

					if (InReadMaskData)
					{
						TFuture<FImage> LayerMaskTask = InReadMaskData();
						LayerMaskTask.Wait();

						const FImage& InMaskImage = LayerMaskTask.Get();

						if (InMaskImage.GetWidth() > 0 && InMaskImage.GetHeight() > 0)
						{
							UTexture2D* MaskTexture = Layer->Mask.LoadSynchronous();
							if (!MaskTexture || !MaskTexture->IsAsset())
							{
								MaskTexture = Cast<UTexture2D>(
									MakeAsset(UTexture2D::StaticClass(),
										Document->GetName(),
										FString::Printf(TEXT("%s_%i_Mask"), *Layer->Id.Name, Layer->Id.Index),
										TEXT("T")));
							}

							if (!MaskTexture)
							{
								UE_LOGF(LogPSDImporterEditor, Error, "Error creating mask texture for layer");
							}
							else
							{
								Layer->Mask = MaskTexture;

								MaskTexture->PreEditChange(nullptr);
								MaskTexture->Source.Init(InMaskImage);
								MaskTexture->Source.Compress();

								UPSDLayerTextureUserData* MaskTextureUserData = MaskTexture->GetAssetUserData<UPSDLayerTextureUserData>();
								if (!MaskTextureUserData)
								{
									MaskTextureUserData = NewObject<UPSDLayerTextureUserData>(MaskTexture, NAME_None, RF_Public | RF_Transactional);
									MaskTexture->AddAssetUserData(MaskTextureUserData);
								}

								MaskTextureUserData->LayerId = Layer->Id;

								MaskTextureUserData->NormalizedBounds = FBox2D(FVector2d(Layer->MaskBounds.Min) / Document->Size, FVector2d(Layer->MaskBounds.Max) / Document->Size);

								MakeValid(MaskTextureUserData->NormalizedBounds.Min.X);
								MakeValid(MaskTextureUserData->NormalizedBounds.Min.Y);
								MakeValid(MaskTextureUserData->NormalizedBounds.Max.X);
								MakeValid(MaskTextureUserData->NormalizedBounds.Max.Y);

								MaskTextureUserData->PixelBounds = Layer->Bounds;

								MaskTexture->PostEditChange();
							}
						}
					}
				}
			}

			// Thumbnail
			{
				constexpr int32 MinThumbnailSize = 48;
				constexpr int32 MaxThumbnailSize = 256;

				const FIntPoint ThumbnailSize = UE::PSDImporterEditor::Private::FitMinClampMaxXY(
					FIntPoint(InLayerImage.GetWidth(), InLayerImage.GetHeight()),
					MinThumbnailSize,
					MaxThumbnailSize
				);

				FImage ThumbnailImage;
				ThumbnailImage.Init(ThumbnailSize.X, ThumbnailSize.Y, ERawImageFormat::BGRA8, InLayerImage.GetGammaSpace());

				if (ThumbnailImage.GetNumPixels() > 0)
				{
					InLayerImage.ResizeTo(
						ThumbnailImage,
						ThumbnailSize.X,
						ThumbnailSize.Y,
						InLayerImage.Format,
						InLayerImage.GetGammaSpace());

					UTexture2D* ThumbnailTexture = Layer->ThumbnailTexture;
					if (!Layer->ThumbnailTexture)
					{
						ThumbnailTexture = Layer->ThumbnailTexture = NewObject<UTexture2D>(Document.Get());
					}

					ThumbnailTexture->PreEditChange(nullptr);
					ThumbnailTexture->Source.Init(ThumbnailImage);
					ThumbnailTexture->Source.Compress();
					ThumbnailTexture->PostEditChange();
				}
			}
		}
	}

	NewLayers.Add(MoveTemp(*Layer));
}

UObject* FPSDDocumentImportFactory_Visitors::MakeAsset(UClass* InClass, const FString& InDocumentName, const FString& InAssetName, const FString& InAssetPrefix)
{
	if (!Document.IsValid())
	{
		return nullptr;
	}

	using namespace UE::PSDImporter::Private;

	FString BasePath;
	FString AssetName;
	MakeAssetPath(InDocumentName, InAssetName, InAssetPrefix, BasePath, AssetName);

	SanitizePackagePath(BasePath);
	SanitizeAssetName(AssetName);

	FString AssetPath = BasePath / AssetName + TEXT(".") + AssetName;
	const bool bAlreadyImported = ImportedAssets.Contains(AssetPath);
	ImportedAssets.Add(AssetPath);

	// If not already imported by us, the CreateAsset call will ask the user if they want to overwrite 
	// an existing asset.
	if (bAlreadyImported)
	{
		TSoftObjectPtr<UObject> ExistingObjectPtr = TSoftObjectPtr<UObject>(FSoftObjectPath(*AssetPath));

		if (UObject* ExistingObject = ExistingObjectPtr.LoadSynchronous())
		{
			return ExistingObject;
		}
	}

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();

	FString PackageName;
	return AssetTools.CreateAsset(AssetName, BasePath, InClass, nullptr);
}

void FPSDDocumentImportFactory_Visitors::MakeAssetPath(
	const FString& InDocumentName, const FString& InAssetName, const FString& InAssetPrefix,
	FString& OutPath, FString& OutName) const
{
	FString BasePath = Document->GetPackage()->GetPathName() + TEXT("_Layers");
	FString AssetName;

	if (InAssetPrefix.IsEmpty())
	{
		AssetName = FString::Printf(TEXT("%s_%s"), *InDocumentName, *InAssetName);
	}
	else
	{
		AssetName = FString::Printf(TEXT("%s_%s_%s"), *InAssetPrefix, *InDocumentName, *InAssetName);
	}

	OutPath = BasePath;
	OutName = ObjectTools::SanitizeInvalidChars(AssetName, INVALID_LONGPACKAGE_CHARACTERS TEXT("/"));
}
