// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PSDFileImport.h"

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "PSDFile.h"
#include "UObject/StrongObjectPtr.h"

class UPSDDocument;

struct FPSDDocumentImportFactory_Visitors : UE::PSDImporter::FPSDFileImportVisitors
{
	explicit FPSDDocumentImportFactory_Visitors(const FString& InFilePath, UPSDDocument* InDocument);

	virtual void OnImportComplete() override;

	virtual void OnImportHeader(const FHeaderInputType& InHeader) override;

	virtual void OnImportLayers(const FLayersInputType& InLayers) override;

	virtual void OnImportLayer(const FLayerInputType& InLayer, const FLayerInputType* InParentLayer, 
		TFunction<TFuture<FImage>()> InReadLayerData, TFunction<TFuture<FImage>()> InReadMaskData) override;

private:
	UObject* MakeAsset(UClass* InClass, const FString& InDocumentName, const FString& InAssetName, const FString& InAssetPrefix);

	void MakeAssetPath(const FString& InDocumentName, const FString& InAssetName, const FString& InAssetPrefix,
		FString& OutPath, FString& OutName) const;

	FString FilePath;
	TStrongObjectPtr<UPSDDocument> Document;
	FPSDFileDocument& FileDocument;
	TSet<FPSDFileLayer> OldLayers;
	TSet<FPSDFileLayer> NewLayers;
	TSet<FString> ImportedAssets;
};
