// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/PSDImporterLayeredMaterialFactory.h"

#include "AssetToolsModule.h"
#include "Containers/Set.h"
#include "IAssetTools.h"
#include "MaterialDomain.h"
#include "MaterialExpressionIO.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Misc/ScopedSlowTask.h"
#include "PSDDocument.h"
#include "PSDImporterEditorLog.h"
#include "PSDImporterEditorUtilities.h"
#include "PSDQuadMeshActor.h"
#include "Utils/PSDImporterMaterialLibrary.h"
#include "Utils/PSDImporterMaterialNodeArranger.h"

#define LOCTEXT_NAMESPACE "PSDImporterLayeredMaterialFactory"

namespace UE::PSDImporter::Private
{
	constexpr const TCHAR* LayerNoCropFunctionPath = TEXT("/Script/Engine.MaterialFunction'/PSDImporter/PSDImporter/MF_PSDImporter_Layer_NoCrop.MF_PSDImporter_Layer_NoCrop'");
	constexpr const TCHAR* LayerCropFunctionPath = TEXT("/Script/Engine.MaterialFunction'/PSDImporter/PSDImporter/MF_PSDImporter_Layer_Crop.MF_PSDImporter_Layer_Crop'");
	constexpr const TCHAR* LayerNoCropMaskFunctionPath = TEXT("/Script/Engine.MaterialFunction'/PSDImporter/PSDImporter/MF_PSDImporter_Layer_NoCrop_Mask.MF_PSDImporter_Layer_NoCrop_Mask'");
	constexpr const TCHAR* LayerCropMaskFunctionPath = TEXT("/Script/Engine.MaterialFunction'/PSDImporter/PSDImporter/MF_PSDImporter_Layer_Crop_Mask.MF_PSDImporter_Layer_Crop_Mask'");
}

bool UPSDImporterLayeredMaterialFactory::CanCreateMaterial(const UPSDDocument* InDocument) const
{
	if (!InDocument)
	{
		return false;
	}

	return InDocument->GetTextureCount() <= UE::PSDImporter::MaxSamplerCount;
}

UMaterial* UPSDImporterLayeredMaterialFactory::CreateMaterial(const UPSDDocument* InDocument) const
{
	if (!IsValid(InDocument))
	{
		UE_LOGF(LogPSDImporterEditor, Error, "Invalid material..");
		return nullptr;
	}

	FScopedSlowTask SlowTask(2.f + InDocument->GetLayers().Num(), LOCTEXT("ImportingPSDFile", "Importing PSD file..."));
	SlowTask.MakeDialog();
	SlowTask.EnterProgressFrame(1.f, LOCTEXT("CreatingMaterial", "Creating Material..."));

	UMaterial* Material = CreateMaterialAsset(*InDocument);

	if (!Material)
	{
		UE_LOGF(LogPSDImporterEditor, Error, "Failed to create layered material.");
		return nullptr;
	}

	const FText LayerPrompt = LOCTEXT("ImportingPSDLayers", "Importing layer data...");

	UMaterialEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		UE_LOGF(LogPSDImporterEditor, Error, "Missing editor only data.");
		return nullptr;
	}

	Material->MaterialDomain = EMaterialDomain::MD_Surface;
	Material->BlendMode = EBlendMode::BLEND_Translucent;

	FExpressionInput& BaseColor = EditorOnlyData->BaseColor;
	FExpressionInput& Opacity = EditorOnlyData->Opacity;

	if (UMaterialExpression* RootExpression = CreateLayers(*Material, *InDocument))
	{
		SlowTask.EnterProgressFrame(1.f, LayerPrompt);

		BaseColor.Connect(0, RootExpression);

		// There was only a single layer
		if (RootExpression->IsA<UMaterialExpressionTextureSample>())
		{
			// Connect the A output of the texture sample
			Opacity.Connect(4, RootExpression);
		}
		else if (RootExpression->IsA<UMaterialExpressionMaterialFunctionCall>())
		{
			// Connect the Opacity output of the (No)Crop function
			Opacity.Connect(1, RootExpression);
		}

		FPSDImporterMaterialNodeArranger::ArrangeNodes(BaseColor);

		UE::PSDImporterEditor::Private::AddOpacityParameterNodes(*Material);

		FPSDImporterMaterialNodeArranger::ArrangeNodes(Opacity);
	}

	SlowTask.EnterProgressFrame(1.f, LOCTEXT("CompilingMaterial", "Compiling Material..."));

	// Force the material to compile
	Material->PreEditChange(nullptr);
	Material->PostEditChange();

	return Material;
}

UMaterial* UPSDImporterLayeredMaterialFactory::CreateMaterialAsset(const UPSDDocument& InDocument) const
{
	const FString BasePath = FPaths::GetPath(InDocument.GetPackage()->GetPathName());
	FString AssetName = InDocument.GetName();

	if (AssetName.StartsWith(TEXT("PSD_")))
	{
		AssetName = TEXT("M_") + AssetName.RightChop(4);
	}
	else
	{
		AssetName = TEXT("M_") + AssetName;
	}

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	FString PackageName;
	AssetTools.CreateUniqueAssetName(BasePath / AssetName, TEXT(""), PackageName, AssetName);

	return Cast<UMaterial>(AssetTools.CreateAsset(AssetName, FPaths::GetPath(PackageName), UMaterial::StaticClass(), nullptr));
}

UMaterialExpression* UPSDImporterLayeredMaterialFactory::CreateLayers(UMaterial& InMaterial, const UPSDDocument& InDocument) const
{
	TArray<const FPSDFileLayer*> ValidLayers = InDocument.GetValidLayers();

	if (ValidLayers.IsEmpty())
	{
		return nullptr;
	}

	UMaterialExpressionConstant4Vector* BaseColor = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant4Vector>(InMaterial);

	if (!BaseColor)
	{
		return nullptr;
	}

	BaseColor->Constant = FLinearColor::Black;
	BaseColor->Constant.A = 0.f;

	if (ValidLayers.Num() == 0)
	{
		return BaseColor;
	}

	UMaterialExpressionMaterialFunctionCall* CurrentLayer = CreateLayer(InMaterial, InDocument, *ValidLayers[0]);

	if (!CurrentLayer)
	{
		return BaseColor;
	}

	// Connect the RGBA output of the base color to the first input.
	CurrentLayer->FunctionInputs[0].Input.Connect(0, BaseColor);

	// Connect the A output of the base color to the second input.
	CurrentLayer->FunctionInputs[1].Input.Connect(4, BaseColor);

	for (int32 Index = 1; Index < ValidLayers.Num(); ++Index)
	{
		UMaterialExpressionMaterialFunctionCall* NextLayer = CreateLayer(InMaterial, InDocument, *ValidLayers[Index]);

		if (!NextLayer)
		{
			break;
		}

		// Connect the Color output of the (No)Crop function to the first input.
		NextLayer->FunctionInputs[0].Input.Connect(0, CurrentLayer);

		// Connect the Opacity output of the (No)Crop function to the second input.
		NextLayer->FunctionInputs[1].Input.Connect(1, CurrentLayer);

		CurrentLayer = NextLayer;
	}

	return CurrentLayer;
}

UMaterialExpressionMaterialFunctionCall* UPSDImporterLayeredMaterialFactory::CreateLayer(UMaterial& InMaterial, const UPSDDocument& InDocument, 
	const FPSDFileLayer& InLayer) const
{
	const FIntPoint& LayerSize = InDocument.GetSize();
	const FIntRect& LayerBounds = InLayer.Bounds;
	const FIntRect& MaskBounds = InLayer.MaskBounds;
	const bool bHasMask = InLayer.HasMask();

	if (InDocument.WereLayersResizedOnImport() || !InLayer.NeedsCrop(LayerSize))
	{
		if (bHasMask)
		{
			return CreateLayer_NoCrop_Mask(InMaterial, InLayer);	
		}
		else
		{
			return CreateLayer_NoCrop(InMaterial, InLayer);
		}
	}
	else
	{
		if (bHasMask)
		{
			return CreateLayer_Crop_Mask(InMaterial, InLayer, LayerSize, LayerBounds, MaskBounds);
		}
		else
		{
			return CreateLayer_Crop(InMaterial, InLayer, LayerSize, LayerBounds);
		}
	}
}

UMaterialExpressionTextureSample* UPSDImporterLayeredMaterialFactory::CreateLayer_Base(UMaterial& InMaterial, const FPSDFileLayer& InLayer) const
{
	using namespace UE::PSDImporter::Private;

	UMaterialExpressionTextureSample* TextureSample = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionTextureSample>(InMaterial);
	TextureSample->Texture = InLayer.Texture.LoadSynchronous();

	return TextureSample;
}

UMaterialExpressionMaterialFunctionCall* UPSDImporterLayeredMaterialFactory::CreateLayer_NoCrop(UMaterial& InMaterial, const FPSDFileLayer& InLayer) const
{
	using namespace UE::PSDImporter::Private;

	UMaterialFunctionInterface* NoCropFunction = FPSDImporterMaterialLibrary::GetMaterialFunction(LayerNoCropFunctionPath);

	if (!NoCropFunction)
	{
		return nullptr;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionMaterialFunctionCall>(InMaterial);
	FunctionCall->SetMaterialFunction(NoCropFunction);
	FunctionCall->UpdateFromFunctionResource();

	// First 2 inputs are for the previous layer.

	UMaterialExpressionTextureObject* LayerTexture = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionTextureObject>(InMaterial);
	LayerTexture->Texture = InLayer.Texture.LoadSynchronous();

	FunctionCall->FunctionInputs[2].Input.Connect(0, LayerTexture);

	return FunctionCall;
}

UMaterialExpressionMaterialFunctionCall* UPSDImporterLayeredMaterialFactory::CreateLayer_Crop(UMaterial& InMaterial, 
	const FPSDFileLayer& InLayer, const FIntPoint& InSize, const FIntRect& InBounds) const
{
	using namespace UE::PSDImporter::Private;

	UMaterialFunctionInterface* CropFunction = FPSDImporterMaterialLibrary::GetMaterialFunction(LayerCropFunctionPath);

	if (!CropFunction)
	{
		return nullptr;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionMaterialFunctionCall>(InMaterial);
	FunctionCall->SetMaterialFunction(CropFunction);
	FunctionCall->UpdateFromFunctionResource();

	// First 2 inputs are for the previous layer.

	UMaterialExpressionTextureObject* LayerTexture = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionTextureObject>(InMaterial);
	LayerTexture->Texture = InLayer.Texture.LoadSynchronous();

	// Connect the texture object to the layer texture input.
	FunctionCall->FunctionInputs[2].Input.Connect(0, LayerTexture);

	UMaterialExpressionConstant2Vector* Position = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant2Vector>(InMaterial);
	Position->R = static_cast<float>(InBounds.Min.X) / static_cast<float>(InSize.X);
	Position->G = static_cast<float>(InBounds.Min.Y) / static_cast<float>(InSize.Y);

	// Connect the position node to the layer size input
	FunctionCall->FunctionInputs[3].Input.Connect(0, Position);

	UMaterialExpressionConstant2Vector* Size = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant2Vector>(InMaterial);
	Size->R = static_cast<float>(InBounds.Width()) / static_cast<float>(InSize.X);
	Size->G = static_cast<float>(InBounds.Height()) / static_cast<float>(InSize.Y);

	// Connect the position node to the layer size input
	FunctionCall->FunctionInputs[4].Input.Connect(0, Size);

	return FunctionCall;
}

UMaterialExpressionMaterialFunctionCall* UPSDImporterLayeredMaterialFactory::CreateLayer_NoCrop_Mask(UMaterial& InMaterial,
	const FPSDFileLayer& InLayer) const
{
	using namespace UE::PSDImporter::Private;

	UMaterialFunctionInterface* NoCropMaskFunction = FPSDImporterMaterialLibrary::GetMaterialFunction(LayerNoCropMaskFunctionPath);

	if (!NoCropMaskFunction)
	{
		return nullptr;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionMaterialFunctionCall>(InMaterial);
	FunctionCall->SetMaterialFunction(NoCropMaskFunction);
	FunctionCall->UpdateFromFunctionResource();

	// First 2 inputs are for the previous layer.

	UMaterialExpressionTextureObject* LayerTexture = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionTextureObject>(InMaterial);
	LayerTexture->Texture = InLayer.Texture.LoadSynchronous();

	FunctionCall->FunctionInputs[2].Input.Connect(0, LayerTexture);

	UMaterialExpressionTextureObject* MaskTexture = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionTextureObject>(InMaterial);
	MaskTexture->Texture = InLayer.Mask.LoadSynchronous();

	FunctionCall->FunctionInputs[3].Input.Connect(0, MaskTexture);

	UMaterialExpressionConstant* MaskDefaultValue = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant>(InMaterial);
	MaskDefaultValue->R = InLayer.MaskDefaultValue;

	FunctionCall->FunctionInputs[4].Input.Connect(0, MaskDefaultValue);

	return FunctionCall;
}

UMaterialExpressionMaterialFunctionCall* UPSDImporterLayeredMaterialFactory::CreateLayer_Crop_Mask(UMaterial& InMaterial,
	const FPSDFileLayer& InLayer, const FIntPoint& InLayerSize, const FIntRect& InLayerBounds, const FIntRect& InMaskBounds) const
{
	using namespace UE::PSDImporter::Private;

	UMaterialFunctionInterface* CropMaskFunction = FPSDImporterMaterialLibrary::GetMaterialFunction(LayerCropMaskFunctionPath);

	if (!CropMaskFunction)
	{
		return nullptr;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionMaterialFunctionCall>(InMaterial);
	FunctionCall->SetMaterialFunction(CropMaskFunction);
	FunctionCall->UpdateFromFunctionResource();

	// First 2 inputs are for the previous layer.

	UMaterialExpressionTextureObject* LayerTexture = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionTextureObject>(InMaterial);
	LayerTexture->Texture = InLayer.Texture.LoadSynchronous();

	FunctionCall->FunctionInputs[2].Input.Connect(0, LayerTexture);

	UMaterialExpressionConstant2Vector* LayerPosition = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant2Vector>(InMaterial);
	LayerPosition->R = static_cast<float>(InLayerBounds.Min.X) / static_cast<float>(InLayerSize.X);
	LayerPosition->G = static_cast<float>(InLayerBounds.Min.Y) / static_cast<float>(InLayerSize.Y);

	// Connect the position node to the layer size input
	FunctionCall->FunctionInputs[3].Input.Connect(0, LayerPosition);

	UMaterialExpressionConstant2Vector* LayerSize = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant2Vector>(InMaterial);
	LayerSize->R = static_cast<float>(InLayerBounds.Width()) / static_cast<float>(InLayerSize.X);
	LayerSize->G = static_cast<float>(InLayerBounds.Height()) / static_cast<float>(InLayerSize.Y);

	// Connect the position node to the layer size input
	FunctionCall->FunctionInputs[4].Input.Connect(0, LayerSize);

	UMaterialExpressionTextureObject* MaskTexture = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionTextureObject>(InMaterial);
	MaskTexture->Texture = InLayer.Mask.LoadSynchronous();

	FunctionCall->FunctionInputs[5].Input.Connect(0, MaskTexture);

	UMaterialExpressionConstant2Vector* MaskPosition = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant2Vector>(InMaterial);
	MaskPosition->R = static_cast<float>(InMaskBounds.Min.X) / static_cast<float>(InLayerSize.X);
	MaskPosition->G = static_cast<float>(InMaskBounds.Min.Y) / static_cast<float>(InLayerSize.Y);

	// Connect the position node to the layer size input
	FunctionCall->FunctionInputs[6].Input.Connect(0, MaskPosition);

	UMaterialExpressionConstant2Vector* MaskSize = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant2Vector>(InMaterial);
	MaskSize->R = static_cast<float>(InMaskBounds.Width()) / static_cast<float>(InLayerSize.X);
	MaskSize->G = static_cast<float>(InMaskBounds.Height()) / static_cast<float>(InLayerSize.Y);

	// Connect the position node to the layer size input
	FunctionCall->FunctionInputs[7].Input.Connect(0, MaskSize);

	UMaterialExpressionConstant* MaskDefaultValue = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionConstant>(InMaterial);
	MaskDefaultValue->R = InLayer.MaskDefaultValue;

	FunctionCall->FunctionInputs[8].Input.Connect(0, MaskDefaultValue);

	return FunctionCall;
}

#undef LOCTEXT_NAMESPACE
