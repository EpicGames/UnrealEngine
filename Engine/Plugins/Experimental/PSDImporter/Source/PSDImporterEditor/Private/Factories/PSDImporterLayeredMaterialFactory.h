// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Math/MathFwd.h"

#include "PSDImporterLayeredMaterialFactory.generated.h"

class UMaterial;
class UMaterialExpression;
class UMaterialExpressionMaterialFunctionCall;
class UMaterialExpressionTextureSample;
class UPSDDocument;
struct FExpressionInput;
struct FPSDFileLayer;

UCLASS()
class UPSDImporterLayeredMaterialFactory : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UPSDImporterLayeredMaterialFactory() override = default;

	bool CanCreateMaterial(const UPSDDocument* InDocument) const;

	UMaterial* CreateMaterial(const UPSDDocument* InDocument) const;

protected:
	UMaterial* CreateMaterialAsset(const UPSDDocument& InDocument) const;

	UMaterialExpression* CreateLayers(UMaterial& InMaterial, const UPSDDocument& InDocument) const;

	UMaterialExpressionMaterialFunctionCall* CreateLayer(UMaterial& InMaterial, const UPSDDocument& InDocument, const FPSDFileLayer& InLayer) const;
	UMaterialExpressionTextureSample* CreateLayer_Base(UMaterial& InMaterial, const FPSDFileLayer& InLayer) const;
	UMaterialExpressionMaterialFunctionCall* CreateLayer_NoCrop(UMaterial& InMaterial, const FPSDFileLayer& InLayer) const;
	UMaterialExpressionMaterialFunctionCall* CreateLayer_Crop(UMaterial& InMaterial, const FPSDFileLayer& InLayer, const FIntPoint& InSize,
		const FIntRect& InBounds) const;
	UMaterialExpressionMaterialFunctionCall* CreateLayer_NoCrop_Mask(UMaterial& InMaterial, const FPSDFileLayer& InLayer) const;
	UMaterialExpressionMaterialFunctionCall* CreateLayer_Crop_Mask(UMaterial& InMaterial, const FPSDFileLayer& InLayer, const FIntPoint& InLayerSize,
		const FIntRect& InLayerBounds, const FIntRect& InMaskBounds) const;
};
