// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/PSDQuadsFactory.h"

#include "Engine/World.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/ScopedSlowTask.h"
#include "PSDDocument.h"
#include "PSDImporterEditorUtilities.h"
#include "PSDQuadActor.h"
#include "PSDQuadMeshActor.h"

#define LOCTEXT_NAMESPACE "PSDQuadsFactory"

namespace UE::PSDImporter::Private
{
	constexpr const TCHAR* MaterialNoMaskPath = TEXT("/Script/Engine.Material'/PSDImporter/PSDImporter/M_PSDImporter_Quad.M_PSDImporter_Quad'");
	constexpr const TCHAR* MaterialMaskPath = TEXT("/Script/Engine.Material'/PSDImporter/PSDImporter/M_PSDImporter_Quad_Mask.M_PSDImporter_Quad_Mask'");
	constexpr const TCHAR* MaterialClippingPath = TEXT("/Script/Engine.Material'/PSDImporter/PSDImporter/M_PSDImporter_Quad_Clipping.M_PSDImporter_Quad_Clipping'");
	constexpr const TCHAR* MaterialMaskClippingPath = TEXT("/Script/Engine.Material'/PSDImporter/PSDImporter/M_PSDImporter_Quad_Mask_Clipping.M_PSDImporter_Quad_Mask_Clipping'");
	constexpr const TCHAR* MaterialClippingClipMaskPath = TEXT("/Script/Engine.Material'/PSDImporter/PSDImporter/M_PSDImporter_Quad_Clipping_ClippingMask.M_PSDImporter_Quad_Clipping_ClippingMask'");
	constexpr const TCHAR* MaterialMaskClippingClipMaskPath = TEXT("/Script/Engine.Material'/PSDImporter/PSDImporter/M_PSDImporter_Quad_Mask_Clipping_ClippingMask.M_PSDImporter_Quad_Mask_Clipping_ClippingMask'");

	UMaterial* GetQuadMaterial(EPSDImporterLayerMaterialType InLayerType)
	{
		TSoftObjectPtr<UMaterial> MaterialPtr;

		switch (InLayerType)
		{
			case EPSDImporterLayerMaterialType::Default:
				MaterialPtr = FSoftObjectPath(MaterialNoMaskPath);
				break;

			case EPSDImporterLayerMaterialType::HasMask:
				MaterialPtr = FSoftObjectPath(MaterialMaskPath);
				break;

			case EPSDImporterLayerMaterialType::IsClipping:
				MaterialPtr = FSoftObjectPath(MaterialClippingPath);
				break;

			case EPSDImporterLayerMaterialType::IsClipping | EPSDImporterLayerMaterialType::HasMask:
				MaterialPtr = FSoftObjectPath(MaterialMaskClippingPath);
				break;

			case EPSDImporterLayerMaterialType::IsClipping | EPSDImporterLayerMaterialType::ClipHasMask:
				MaterialPtr = FSoftObjectPath(MaterialClippingClipMaskPath);
				break;

			case EPSDImporterLayerMaterialType::IsClipping | EPSDImporterLayerMaterialType::HasMask | EPSDImporterLayerMaterialType::ClipHasMask:
				MaterialPtr = FSoftObjectPath(MaterialMaskClippingClipMaskPath);
				break;

			default:
				return nullptr;
		}

		return MaterialPtr.LoadSynchronous();
	}
}

APSDQuadActor* UPSDQuadsFactory::CreateQuadActor(UWorld& InWorld, UPSDDocument& InDocument) const
{
	using namespace UE::PSDImporterEditor::Private;

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.bNoFail = true;

	APSDQuadActor* QuadActor = InWorld.SpawnActor<APSDQuadActor>(SpawnParams);
	QuadActor->SetPSDDocument(InDocument);
	QuadActor->SetActorScale3D(FVector(InitialScale, InitialScale, InitialScale));

	return QuadActor;
}

void UPSDQuadsFactory::CreateQuads(APSDQuadActor& InQuadActor) const
{
	UPSDDocument* Document = InQuadActor.GetPSDDocument();

	if (!Document)
	{
		return;
	}

	const TArray<FPSDFileLayer>& Layers = Document->GetLayers();

	FScopedSlowTask SlowTask(Layers.Num(), LOCTEXT("CreatingPSDQuads", "Creating PSD Quads..."));
	SlowTask.MakeDialog();

	for (int32 Index = 0; Index < Layers.Num(); ++Index)
	{
		SlowTask.EnterProgressFrame(1.f);

		const FPSDFileLayer& Layer = Layers[Index];

		if (!Layer.bIsVisible || !Layer.bIsSupportedLayerType
			|| FMath::IsNearlyZero(Layer.Opacity)
			|| (Layer.Bounds.Area() == 0))
		{
			continue;
		}

		// Skip the layer if the next layer is a clipping layer, it will be taken into account by the clipping layer.
		if (Layers.IsValidIndex(Index + 1))
		{
			const FPSDFileLayer& NextLayer = Layers[Index + 1];

			if (NextLayer.Clipping > 0)
			{
				continue;
			}
		}

		if (APSDQuadMeshActor* QuadMesh = CreateQuad(InQuadActor, Index))
		{
			InQuadActor.AddQuadMesh(*QuadMesh);
		}
	}

	InQuadActor.InitComplete();
}

APSDQuadMeshActor* UPSDQuadsFactory::CreateQuad(APSDQuadActor& InQuadActor, int32 InLayerIndex) const
{
	UPSDDocument* Document = InQuadActor.GetPSDDocument();

	if (!Document)
	{
		return nullptr;
	}

	if (!Document->GetLayers().IsValidIndex(InLayerIndex))
	{
		return nullptr;
	}

	UWorld* World = InQuadActor.GetWorld();

	if (!World)
	{
		return nullptr;
	}

	FActorSpawnParameters Params;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	Params.bNoFail = true;

	if (InQuadActor.bIsEditorPreviewActor)
	{
		Params.bTemporaryEditorActor = true;
		Params.bHideFromSceneOutliner = true;
	}

	APSDQuadMeshActor* QuadMesh = World->SpawnActor<APSDQuadMeshActor>(Params);

	if (!QuadMesh)
	{
		return nullptr;
	}

	using namespace UE::PSDImporter::Private;
	using namespace UE::PSDImporterEditor::Private;

	const EPSDImporterLayerMaterialType LayerType = GetLayerMaterialType(Document->GetLayers(), InLayerIndex);

	UMaterialInstanceDynamic* LayerMaterial = UMaterialInstanceDynamic::Create(GetQuadMaterial(LayerType), QuadMesh);

	QuadMesh->InitLayer(InQuadActor, InLayerIndex, LayerMaterial);

	if (InQuadActor.bIsEditorPreviewActor)
	{
		QuadMesh->SetActorEnableCollision(false);
	}

	return QuadMesh;
}

#undef LOCTEXT_NAMESPACE
