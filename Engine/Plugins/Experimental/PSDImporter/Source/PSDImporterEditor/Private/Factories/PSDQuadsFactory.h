// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "PSDQuadsFactory.generated.h"

class APSDQuadActor;
class APSDQuadMeshActor;
class UPSDDocument;
class UWorld;

UCLASS()
class UPSDQuadsFactory : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UPSDQuadsFactory() override = default;

	APSDQuadActor* CreateQuadActor(UWorld& InWorld, UPSDDocument& InDocument) const;

	void CreateQuads(APSDQuadActor& InQuadActor) const;

protected:
	APSDQuadMeshActor* CreateQuad(APSDQuadActor& InQuadActor, int32 InLayerIndex) const;
};
