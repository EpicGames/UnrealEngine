// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterContentBrowserIntegration.h"

#include "ContentBrowserModule.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "Factories/PSDImporterLayeredMaterialFactory.h"
#include "Factories/PSDQuadsFactory.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Misc/MessageDialog.h"
#include "PSDDocument.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "FPSDImporterContentBrowserIntegration"

FPSDImporterContentBrowserIntegration& FPSDImporterContentBrowserIntegration::Get()
{
	static FPSDImporterContentBrowserIntegration Object;
	return Object;
}

void FPSDImporterContentBrowserIntegration::Integrate()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	CBMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateRaw(this, &FPSDImporterContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu));
	ContentBrowserHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

void FPSDImporterContentBrowserIntegration::Disintegrate()
{
	if (!ContentBrowserHandle.IsValid())
	{
		return;
	}

	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule->GetAllAssetViewContextMenuExtenders();

		CBMenuExtenderDelegates.RemoveAll(
			[this](const FContentBrowserMenuExtender_SelectedAssets& InElement)
			{
				return InElement.GetHandle() == ContentBrowserHandle;
			}
		);
	}

	ContentBrowserHandle.Reset();
}

TSharedRef<FExtender> FPSDImporterContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	bool bHasDocument = false;

	for (const FAssetData& SelectedAsset : InSelectedAssets)
	{
		if (UClass* AssetClass = SelectedAsset.GetClass(EResolveClass::Yes))
		{
			if (AssetClass->IsChildOf<UPSDDocument>())
			{
				bHasDocument = true;
				break;
			}
		}
	}

	if (!bHasDocument)
	{
		return Extender;
	}

	Extender->AddMenuExtension(
		"GetAssetActions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateRaw(this, &FPSDImporterContentBrowserIntegration::CreateMenuEntries, InSelectedAssets)
	);

	return Extender;
}

void FPSDImporterContentBrowserIntegration::CreateMenuEntries(FMenuBuilder& InMenuBuilder, TArray<FAssetData> InSelectedAssets)
{
	InMenuBuilder.BeginSection(TEXT("PSDDocument"), LOCTEXT("PSDDocument", "PSD Document"));

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CreatePSDDocumentMaterial", "Create Material From PSD Document"),
		LOCTEXT("CreatePSDDocumentMaterialTooltip", "Creates a material comprising of the layer stack in the PSD Document."),
		FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()),
		FUIAction(FExecuteAction::CreateRaw(this, &FPSDImporterContentBrowserIntegration::CreatePSDMaterial, InSelectedAssets))
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CreatePSDDocumentQuads", "Create Quads From PSD Document"),
		LOCTEXT("CreatePSDDocumentQuadsTooltip", "Creates a series of quads representing each layer in the PSD Document."),
		FSlateIconFinder::FindIconForClass(UStaticMesh::StaticClass()),
		FUIAction(FExecuteAction::CreateRaw(this, &FPSDImporterContentBrowserIntegration::CreatePSDQuads, InSelectedAssets))
	);

	InMenuBuilder.EndSection();
}

void FPSDImporterContentBrowserIntegration::CreatePSDMaterial(TArray<FAssetData> InSelectedAssets)
{
	UPSDDocument* Document = nullptr;

	for (const FAssetData& AssetData : InSelectedAssets)
	{
		UClass* Class = AssetData.GetClass(EResolveClass::Yes);

		if (!Class || !Class->IsChildOf(UPSDDocument::StaticClass()))
		{
			continue;
		}

		Document = Cast<UPSDDocument>(AssetData.GetAsset());

		if (Document)
		{
			break;
		}
	}

	if (!Document)
	{
		return;
	}

	const UPSDImporterLayeredMaterialFactory* Factory = GetDefault<UPSDImporterLayeredMaterialFactory>();

	if (!Factory->CanCreateMaterial(Document))
	{
		FMessageDialog::Open(
			EAppMsgType::Ok,
			LOCTEXT("ErrorMessage", "PSD document uses too many textures to create a single material."),
			LOCTEXT("ErrorTitle", "Create Material Error")
		);
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CreatePSDMaterial", "Create PSD Material"));
	Factory->CreateMaterial(Document);
}

void FPSDImporterContentBrowserIntegration::CreatePSDQuads(TArray<FAssetData> InSelectedAssets)
{
	if (!GWorld)
	{
		return;
	}

	UPSDDocument* Document = nullptr;

	for (const FAssetData& AssetData : InSelectedAssets)
	{
		UClass* Class = AssetData.GetClass(EResolveClass::Yes);

		if (!Class || !Class->IsChildOf(UPSDDocument::StaticClass()))
		{
			continue;
		}

		Document = Cast<UPSDDocument>(AssetData.GetAsset());

		if (Document)
		{
			break;
		}
	}

	if (!Document)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CreatePSDQuads", "Create PSD Quads"));

	UPSDQuadsFactory* QuadsFactory = GetMutableDefault<UPSDQuadsFactory>();

	if (APSDQuadActor* QuadActor = QuadsFactory->CreateQuadActor(*GWorld, *Document))
	{
		QuadsFactory->CreateQuads(*QuadActor);
	}
}

#undef LOCTEXT_NAMESPACE
