// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

class FExtender;
class FMenuBuilder;

class FPSDImporterContentBrowserIntegration
{
public:
	static FPSDImporterContentBrowserIntegration& Get();

	void Integrate();

	void Disintegrate();

protected:
	TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets);

	void CreateMenuEntries(FMenuBuilder& InMenuBuilder, TArray<FAssetData> InSelectedAssets);

	void CreatePSDMaterial(TArray<FAssetData> InSelectedAssets);

	void CreatePSDQuads(TArray<FAssetData> InSelectedAssets);

	FDelegateHandle ContentBrowserHandle;
};
