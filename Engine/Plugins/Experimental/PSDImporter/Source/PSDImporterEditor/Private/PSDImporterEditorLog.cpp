// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterEditorLog.h"

DEFINE_LOG_CATEGORY(LogPSDImporterEditor);
