// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"

#include "Customizations/PSDActorCustomization.h"
#include "Customizations/PSDLayerCustomization.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/IConsoleManager.h"
#include "Misc/DelayedAutoRegister.h"
#include "Modules/ModuleManager.h"
#include "PSDFile.h"
#include "PSDImporterContentBrowserIntegration.h"
#include "PSDQuadActor.h"
#include "PSDQuadMeshActor.h"
#include "PropertyEditorModule.h"
#include "Utils/PSDImporterMaterialLibrary.h"

#define LOCTEXT_NAMESPACE "PSDImporterEditorModule"

namespace UE::PSDImporterEditor::Private
{
	bool bWasCVarAutomaticallyDisabled = false;
}

class FPSDImporterEditorModule
	: public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(FPSDFileLayer::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::PSDImporterEditor::FPSDLayerCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(APSDQuadActor::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&UE::PSDImporterEditor::FPSDActorCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(APSDQuadMeshActor::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&UE::PSDImporterEditor::FPSDActorCustomization::MakeInstance));

		TSharedRef<FPropertySection> RootLayersSection = PropertyModule.FindOrCreateSection(APSDQuadActor::StaticClass()->GetFName(), TEXT("PSD"), LOCTEXT("PSD", "PSD"));
		RootLayersSection->AddCategory(TEXT("PSD"));

		TSharedRef<FPropertySection> MeshLayersSection = PropertyModule.FindOrCreateSection(APSDQuadMeshActor::StaticClass()->GetFName(), TEXT("PSD"), LOCTEXT("PSD", "PSD"));
		MeshLayersSection->AddCategory(TEXT("PSD"));

		FPSDImporterContentBrowserIntegration::Get().Integrate();

		if (!PostInitHandle.IsValid())
		{
			PostInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(
				[]()
				{
					if (IConsoleVariable* InterchangePSDCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.PSD")))
					{
						using namespace UE::PSDImporterEditor::Private;

						if (InterchangePSDCVar->GetBool())
						{
							bWasCVarAutomaticallyDisabled = true;
							InterchangePSDCVar->Set(false, ECVF_SetByCode);
						}
					}
				}
			);
		}

		if (!PreExitHandle.IsValid())
		{
			PreExitHandle = FCoreDelegates::OnPreExit.AddLambda(
				[]()
				{
					if (IConsoleVariable* InterchangePSDCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.PSD")))
					{
						using namespace UE::PSDImporterEditor::Private;

						if (bWasCVarAutomaticallyDisabled)
						{
							bWasCVarAutomaticallyDisabled = false;
							InterchangePSDCVar->Set(true, ECVF_SetByCode);
						}
					}
				}
			);
		}

		if (!TextureResetHandle.IsValid())
		{
			TextureResetHandle = APSDQuadMeshActor::GetTextureResetDelegate().AddStatic(&FPSDImporterMaterialLibrary::ResetTexture);
		}
	}
	
	virtual void ShutdownModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout(FPSDFileLayer::StaticStruct()->GetFName());

		FPSDImporterContentBrowserIntegration::Get().Disintegrate();

		if (PostInitHandle.IsValid())
		{
			FCoreDelegates::GetOnPostEngineInit().Remove(PostInitHandle);
			PostInitHandle.Reset();
		}

		if (PreExitHandle.IsValid())
		{
			FCoreDelegates::OnPreExit.Remove(PreExitHandle);
			PreExitHandle.Reset();
		}

		if (TextureResetHandle.IsValid())
		{
			APSDQuadMeshActor::GetTextureResetDelegate().Remove(TextureResetHandle);
			TextureResetHandle.Reset();
		}
	}

private:
	FDelegateHandle PostInitHandle;
	FDelegateHandle PreExitHandle;
	FDelegateHandle TextureResetHandle;
};

IMPLEMENT_MODULE(FPSDImporterEditorModule, PSDImporterEditor);

#undef LOCTEXT_NAMESPACE
