// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterEditorSettings.h"

UPSDImporterEditorSettings::UPSDImporterEditorSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("PSD Importer");

	bResizeLayersToDocument = false;
	bImportInvisibleLayers = false;
}

UPSDImporterEditorSettings* UPSDImporterEditorSettings::Get()
{
	return GetMutableDefault<UPSDImporterEditorSettings>();
}
