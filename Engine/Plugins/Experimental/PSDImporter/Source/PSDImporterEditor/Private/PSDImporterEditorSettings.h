// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "PSDImporterEditorSettings.generated.h"

/**
 * DM Texture Set Settings
 */
UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "PSD Importer"))
class UPSDImporterEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPSDImporterEditorSettings();
	virtual ~UPSDImporterEditorSettings() override = default;

	static UPSDImporterEditorSettings* Get();

	UPROPERTY(Config, EditAnywhere, Category = "PSD Importer|Importer Defaults")
	bool bResizeLayersToDocument;

	UPROPERTY(Config, EditAnywhere, Category = "PSD Importer|Importer Defaults")
	bool bImportInvisibleLayers;
};
