// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterEditorUtilities.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialFunction.h"
#include "Utils/PSDImporterMaterialLibrary.h"
#include "PSDQuadMeshActor.h"

namespace UE::PSDImporterEditor
{
	FIntPoint Private::FitMinClampMaxXY(const FIntPoint& InSource, const int32 InMinSize, const int32 InMaxSize)
	{
		double SourceWidthF = static_cast<double>(InSource.X);
		double SourceHeightF = static_cast<double>(InSource.Y);
		
		FVector2D ResultF = FitMinClampMaxXY(FVector2D(SourceWidthF, SourceHeightF), InMinSize, InMaxSize);

		const int32 NewSourceWidth = static_cast<int32>(ResultF.X);
		const int32 NewSourceHeight = static_cast<int32>(ResultF.Y);

		return FIntPoint(NewSourceWidth, NewSourceHeight);
	}

	FVector2D Private::FitMinClampMaxXY(const FVector2D& InSource, const int32 InMinSize, const int32 InMaxSize)
	{
		const double MinSizeF = static_cast<double>(InMinSize);
		const double MaxSizeF = static_cast<double>(InMaxSize);

		double SourceWidthF = static_cast<double>(InSource.X);
		double SourceHeightF = static_cast<double>(InSource.Y);

		const double ScaleToMin = FMath::Min(MinSizeF / SourceWidthF, MinSizeF / SourceHeightF);
		SourceWidthF *= ScaleToMin;
		SourceHeightF *= ScaleToMin;

		if (SourceWidthF > MaxSizeF || SourceHeightF > MaxSizeF)
		{
			const double ScaleToMax = FMath::Min(MaxSizeF / SourceWidthF, MaxSizeF / SourceHeightF);
			SourceWidthF *= ScaleToMax;
			SourceHeightF *= ScaleToMax;
		}

		return FVector2D(SourceWidthF, SourceHeightF);
	}

	void Private::SelectLayerTextureAsset(const FPSDFileLayer& InLayer)
	{
		if (UTexture2D* LayerTexture = InLayer.Texture.LoadSynchronous())
		{
			const TArray<UObject*> ObjectsToSelect = { LayerTexture };
			
			IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser")).Get();
			ContentBrowser.SyncBrowserToAssets(ObjectsToSelect);
		}
	}

	void Private::SelectMaskTextureAsset(const FPSDFileLayer& InLayer)
	{
		if (UTexture2D* MaskTexture = InLayer.Mask.LoadSynchronous())
		{
			const TArray<UObject*> ObjectsToSelect = {MaskTexture};

			IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser")).Get();
			ContentBrowser.SyncBrowserToAssets(ObjectsToSelect);
		}
	}

	bool Private::AddOpacityParameterNodes(UMaterial& InMaterial)
	{
		UMaterialEditorOnlyData* EditorOnlyData = InMaterial.GetEditorOnlyData();

		if (!EditorOnlyData)
		{
			return false;
		}

		FScalarMaterialInput* MaterialInput = nullptr;

		switch (InMaterial.GetBlendMode())
		{
			case BLEND_Translucent:
				MaterialInput = &EditorOnlyData->Opacity;
				break;

			case BLEND_Masked:
				MaterialInput = &EditorOnlyData->OpacityMask;
				break;

			default:
				// Only translucent and masked 
				return false;
		}

		if (!MaterialInput)
		{
			return false;
		}

		// Create global opacity node and fetch pin.
		UMaterialExpressionScalarParameter* GlobalOpacity = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionScalarParameter>(InMaterial);

		if (!GlobalOpacity)
		{
			return false;
		}

		// Create multiply node and fetch pins.
		UMaterialExpressionMultiply* Multiply = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionMultiply>(InMaterial);

		if (!Multiply)
		{
			return false;
		}

		FExpressionInput* MultiplyInputA = Multiply->GetInput(0);
		FExpressionInput* MultiplyInputB = Multiply->GetInput(1);

		if (!MultiplyInputA || !MultiplyInputB)
		{
			return false;
		}

		// Store current connection to the material attributes node.
		UMaterialExpression* CurrentExpression = MaterialInput->Expression;
		const int32 CurrentOutputIndex = MaterialInput->OutputIndex;
		const int32 CurrentMask = MaterialInput->Mask;
		const int32 CurrentMaskR = MaterialInput->MaskR;
		const int32 CurrentMaskG = MaterialInput->MaskG;
		const int32 CurrentMaskB = MaterialInput->MaskB;
		const int32 CurrentMaskA = MaterialInput->MaskA;

		// Connect material attribute
		MaterialInput->Expression = Multiply;
		MaterialInput->OutputIndex = 0;
		MaterialInput->SetMask(0, 0, 0, 0, 0);

		// Connect multiply node
		MultiplyInputA->Expression = CurrentExpression;
		MultiplyInputA->OutputIndex = CurrentOutputIndex;
		MultiplyInputA->SetMask(CurrentMask, CurrentMaskR, CurrentMaskG, CurrentMaskB, CurrentMaskA);

		MultiplyInputB->Expression = GlobalOpacity;
		MultiplyInputB->OutputIndex = 0;

		// Setup global opacity node
		GlobalOpacity->SetParameterName(TEXT("GlobalOpacity"));
		GlobalOpacity->DefaultValue = 1.f;

		if (!FModuleManager::Get().IsModuleLoaded(TEXT("GeometryMask")))
		{
			return true;
		}

		constexpr const TCHAR* GeometryMaskFunctionPath = TEXT("/Script/Engine.MaterialFunction'/PSDImporter/PSDImporter/MF_PSDImporter_ApplyGeometryMask.MF_PSDImporter_ApplyGeometryMask'");

		UMaterialFunction* GeometryMaskFunction = TSoftObjectPtr<UMaterialFunction>(FSoftObjectPath(GeometryMaskFunctionPath)).LoadSynchronous();

		if (!GeometryMaskFunction)
		{
			return true;
		}

		// Create geometry mask node and fetch pins.
		UMaterialExpressionMaterialFunctionCall* GeometryMaskExpression = FPSDImporterMaterialLibrary::CreateExpression<UMaterialExpressionMaterialFunctionCall>(InMaterial);

		if (!GeometryMaskExpression)
		{
			return true;
		}

		GeometryMaskExpression->SetMaterialFunction(GeometryMaskFunction);
		GeometryMaskExpression->UpdateFromFunctionResource();

		const FName OpacityPinName = TEXT("Opacity");

		FExpressionInput* GeometryMaskOpacityInput = nullptr;
		int32 GeometryMaskOpacityOutputIndex = INDEX_NONE;

		for (FExpressionInput* Input : GeometryMaskExpression->GetInputsView())
		{
			if (Input->InputName == OpacityPinName)
			{
				GeometryMaskOpacityInput = Input;
				break;
			}
		}

		if (!GeometryMaskOpacityInput)
		{
			return true;
		}

		const TArray<FExpressionOutput>& GeometryMaskExpressionOutputs = GeometryMaskExpression->GetOutputs();

		for (int32 OutputIndex = 0; OutputIndex < GeometryMaskExpressionOutputs.Num(); ++OutputIndex)
		{
			if (GeometryMaskExpressionOutputs[OutputIndex].OutputName == OpacityPinName)
			{
				GeometryMaskOpacityOutputIndex = OutputIndex;
				break;
			}
		}

		if (GeometryMaskOpacityOutputIndex == INDEX_NONE)
		{
			return true;
		}

		// Connect multiply node
		MultiplyInputA->Expression = GeometryMaskExpression;
		MultiplyInputA->OutputIndex = GeometryMaskOpacityOutputIndex;
		MultiplyInputA->SetMask(0, 0, 0, 0, 0);

		// Connect geomtry mask
		GeometryMaskOpacityInput->Expression = CurrentExpression;
		GeometryMaskOpacityInput->OutputIndex = CurrentOutputIndex;
		GeometryMaskOpacityInput->SetMask(CurrentMask, CurrentMaskR, CurrentMaskG, CurrentMaskB, CurrentMaskA);

		return true;
	}

	EPSDImporterLayerMaterialType Private::GetLayerMaterialType(const TArray<FPSDFileLayer>& InLayers, int32 InLayerIndex)
	{
		EPSDImporterLayerMaterialType LayerType = EPSDImporterLayerMaterialType::Default;

		if (!InLayers.IsValidIndex(InLayerIndex))
		{
			return LayerType;
		}

		const FPSDFileLayer& Layer = InLayers[InLayerIndex];

		if (Layer.HasMask())
		{
			LayerType |= EPSDImporterLayerMaterialType::HasMask;
		}

		if (InLayerIndex > 0 && Layer.Clipping > 0)
		{
			LayerType |= EPSDImporterLayerMaterialType::IsClipping;

			const FPSDFileLayer& PreviousLayer = InLayers[InLayerIndex - 1];

			if (PreviousLayer.HasMask())
			{
				LayerType |= EPSDImporterLayerMaterialType::ClipHasMask;
			}
		}

		return LayerType;
	}
}
