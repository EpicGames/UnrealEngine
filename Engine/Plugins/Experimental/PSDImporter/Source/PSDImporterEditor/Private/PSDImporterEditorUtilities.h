// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PSDFile.h"

class UMaterial;
enum class EPSDImporterLayerMaterialType : uint8;

namespace UE::PSDImporterEditor::Private
{
	// Makes a document at 1080p fit the viewport on initial import.
	constexpr double InitialScale = 0.52;

	// Calculate an X and Y scale such that no one axis is smaller than MinSize, and then none larger than MaxSize.
	// The output can be therefore be smaller than MinSize, but not larger than MaxSize.
	FIntPoint FitMinClampMaxXY(const FIntPoint& InSource, const int32 InMinSize, const int32 InMaxSize);

	FVector2D FitMinClampMaxXY(const FVector2D& InSource, const int32 InMinSize, const int32 InMaxSize);

	void SelectLayerTextureAsset(const FPSDFileLayer& InLayer);

	void SelectMaskTextureAsset(const FPSDFileLayer& InLayer);

	/**
	 * Adds the global opacity parameter and, if it can, the geometry mask apply node.
	 * Returns true if the material attribute input was changed.
	 */
	bool AddOpacityParameterNodes(UMaterial& InMaterial);

	EPSDImporterLayerMaterialType GetLayerMaterialType(const TArray<FPSDFileLayer>& InLayers, int32 InLayerIndex);
}
