// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/PSDImporterMaterialLibrary.h"

#include "Engine/Texture.h"
#include "MaterialEditingLibrary.h"
#include "PSDFile.h"
#include "PSDQuadMeshActor.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

UMaterialFunctionInterface* FPSDImporterMaterialLibrary::GetMaterialFunction(const TCHAR* InFunctionPath)
{
	const TSoftObjectPtr<UMaterialFunctionInterface> FunctionPtr = TSoftObjectPtr<UMaterialFunctionInterface>(FSoftObjectPath(InFunctionPath));
	return FunctionPtr.LoadSynchronous();
}

UMaterialExpression* FPSDImporterMaterialLibrary::CreateExpression(UMaterial& InMaterial, UClass& InExpressionClass)
{
	return UMaterialEditingLibrary::CreateMaterialExpressionEx(
		&InMaterial,
		/* Material Function */ nullptr,
		&InExpressionClass
	);
}

void FPSDImporterMaterialLibrary::ResetTexture(APSDQuadMeshActor& InQuadMeshActor)
{
	UMaterialInterface* Material = InQuadMeshActor.GetQuadMaterial();

	if (!Material)
	{
		return;
	}

	const FPSDFileLayer* Layer = InQuadMeshActor.GetLayer();

	if (!Layer)
	{
		return;
	}

	if (UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(Material))
	{
		MIC->SetTextureParameterValueEditorOnly(UE::PSDImporter::LayerTextureParameterName, Layer->Texture.LoadSynchronous());

		MIC->SetVectorParameterValueEditorOnly(UE::PSDImporter::LayerBoundsParameterName, FLinearColor(
			Layer->Bounds.Min.X, Layer->Bounds.Min.Y, Layer->Bounds.Max.X, Layer->Bounds.Max.Y));

		if (UTexture* MaskTexture = Layer->Mask.LoadSynchronous())
		{
			MIC->SetTextureParameterValueEditorOnly(UE::PSDImporter::MaskTextureParameterName, MaskTexture);

			MIC->SetVectorParameterValueEditorOnly(UE::PSDImporter::MaskBoundsParameterName, FLinearColor(
				Layer->MaskBounds.Min.X, Layer->MaskBounds.Min.Y, Layer->MaskBounds.Max.X, Layer->MaskBounds.Max.Y));

			MIC->SetScalarParameterValueEditorOnly(UE::PSDImporter::MaskDefaultValueParameterName, Layer->MaskDefaultValue);
		}

		if (const FPSDFileLayer* ClippingLayer = InQuadMeshActor.GetClippingLayer())
		{
			MIC->SetTextureParameterValueEditorOnly(UE::PSDImporter::ClippingLayerTextureParameterName, ClippingLayer->Texture.LoadSynchronous());

			MIC->SetVectorParameterValueEditorOnly(UE::PSDImporter::ClippingLayerBoundsParameterName, FLinearColor(
				ClippingLayer->Bounds.Min.X, ClippingLayer->Bounds.Min.Y, ClippingLayer->Bounds.Max.X, ClippingLayer->Bounds.Max.Y));

			if (UTexture* ClipMaskTexture = ClippingLayer->Mask.LoadSynchronous())
			{
				MIC->SetTextureParameterValueEditorOnly(UE::PSDImporter::ClippingMaskTextureParameterName, ClipMaskTexture);

				MIC->SetVectorParameterValueEditorOnly(UE::PSDImporter::ClippingMaskBoundsParameterName, FLinearColor(
					ClippingLayer->MaskBounds.Min.X, ClippingLayer->MaskBounds.Min.Y, ClippingLayer->MaskBounds.Max.X, ClippingLayer->MaskBounds.Max.Y));

				MIC->SetScalarParameterValueEditorOnly(UE::PSDImporter::ClippingMaskDefaultValueParameterName, ClippingLayer->MaskDefaultValue);
			}
		}
	}
	else if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(Material))
	{
		MID->SetTextureParameterValue(UE::PSDImporter::LayerTextureParameterName, Layer->Texture.LoadSynchronous());

		MID->SetVectorParameterValue(UE::PSDImporter::LayerBoundsParameterName, FLinearColor(
			Layer->Bounds.Min.X, Layer->Bounds.Min.Y, Layer->Bounds.Max.X, Layer->Bounds.Max.Y));

		if (UTexture* MaskTexture = Layer->Mask.LoadSynchronous())
		{
			MID->SetTextureParameterValue(UE::PSDImporter::MaskTextureParameterName, MaskTexture);

			MID->SetVectorParameterValue(UE::PSDImporter::MaskBoundsParameterName, FLinearColor(
				Layer->MaskBounds.Min.X, Layer->MaskBounds.Min.Y, Layer->MaskBounds.Max.X, Layer->MaskBounds.Max.Y));

			MID->SetScalarParameterValue(UE::PSDImporter::MaskDefaultValueParameterName, Layer->MaskDefaultValue);
		}

		if (const FPSDFileLayer* ClippingLayer = InQuadMeshActor.GetClippingLayer())
		{
			MID->SetTextureParameterValue(UE::PSDImporter::ClippingLayerTextureParameterName, ClippingLayer->Texture.LoadSynchronous());

			MID->SetVectorParameterValue(UE::PSDImporter::ClippingLayerBoundsParameterName, FLinearColor(
				ClippingLayer->Bounds.Min.X, ClippingLayer->Bounds.Min.Y, ClippingLayer->Bounds.Max.X, ClippingLayer->Bounds.Max.Y));

			if (UTexture* ClipMaskTexture = ClippingLayer->Mask.LoadSynchronous())
			{
				MID->SetTextureParameterValue(UE::PSDImporter::ClippingMaskTextureParameterName, ClipMaskTexture);

				MID->SetVectorParameterValue(UE::PSDImporter::ClippingMaskBoundsParameterName, FLinearColor(
					ClippingLayer->MaskBounds.Min.X, ClippingLayer->MaskBounds.Min.Y, ClippingLayer->MaskBounds.Max.X, ClippingLayer->MaskBounds.Max.Y));

				MID->SetScalarParameterValue(UE::PSDImporter::ClippingMaskDefaultValueParameterName, ClippingLayer->MaskDefaultValue);
			}
		}
	}
}
