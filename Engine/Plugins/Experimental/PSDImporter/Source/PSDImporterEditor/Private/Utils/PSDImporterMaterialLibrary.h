// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class APSDQuadMeshActor;
class UClass;
class UMaterial;
class UMaterialExpression;
class UMaterialFunctionInterface;

class FPSDImporterMaterialLibrary
{
public:
	static UMaterialFunctionInterface* GetMaterialFunction(const TCHAR* InFunctionPath);

	static UMaterialExpression* CreateExpression(UMaterial& InMaterial, UClass& InExpressionClass);

	template<typename InExpressionType>
	static InExpressionType* CreateExpression(UMaterial& InMaterial)
	{
		return Cast<InExpressionType>(CreateExpression(InMaterial, *InExpressionType::StaticClass()));
	}

	static void ResetTexture(APSDQuadMeshActor& InQuadMeshActor);
};
