// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/PSDImporterMaterialNodeArranger.h"

#include "Containers/Set.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Math/IntPoint.h"

namespace UE::PSDImporterEditor::Private
{
	constexpr int32 InitialHorizontalOffset  = -100;
	constexpr int32 HorizontalOffsetPerLayer = 300;
	constexpr int32 FunctionCallOffset       = 300;
	constexpr int32 TextureSampleOffset      = 100;
	constexpr int32 VerticalNodeOffset       = 200;
}

void FPSDImporterMaterialNodeArranger::ArrangeNodes(const FExpressionInput& InMaterialChannelInput)
{
	using namespace UE::PSDImporterEditor::Private;

	if (InMaterialChannelInput.Expression)
	{
		// A little extra room is required, so we start the node arranging slightly to the left of the material attributes.
		ArrangeNodes(*InMaterialChannelInput.Expression, FIntPoint(InitialHorizontalOffset, 0));
	}
}

void FPSDImporterMaterialNodeArranger::ArrangeNodes(UMaterialExpression& InExpression, FIntPoint InPosition)
{
	using namespace UE::PSDImporterEditor::Private;

	// Move the position of the node 300 to the left. This indicates that we're on another horizontal "layer".
	InPosition.X -= HorizontalOffsetPerLayer;

	InExpression.MaterialExpressionEditorX = InPosition.X;
	InExpression.MaterialExpressionEditorY = InPosition.Y;

	// Reset the vertical position to the top.
	InPosition.Y = 0;

	// Avoid arranging the same node twice if we've used it multiple times for the same input.
	TSet<UMaterialExpression*> ArrangedInputs;
	ArrangedInputs.Reserve(InExpression.CountInputs());

	for (FExpressionInputIterator Iter(&InExpression); Iter; ++Iter)
	{
		UMaterialExpression* Expression = Iter->Expression;

		if (!Expression || ArrangedInputs.Contains(Expression))
		{
			continue;
		}

		// Function calls are the base layer, so offset them more to the left
		if (Expression->IsA<UMaterialExpressionMaterialFunctionCall>())
		{
			InPosition.X -= FunctionCallOffset;
		}

		// Texture samples are really high (and are only the first node)
		// So let's just give that a bit of extra room.
		if (Expression->IsA<UMaterialExpressionTextureSample>())
		{
			InPosition.Y -= TextureSampleOffset;
		}

		ArrangeNodes(*Expression, InPosition);

		// Move the position of the next input down by 200.
		InPosition.Y += VerticalNodeOffset;

		// Undo the horizontal offset.
		if (Expression->IsA<UMaterialExpressionMaterialFunctionCall>())
		{
			InPosition.X += FunctionCallOffset;
		}

		// Undo the vertical offset.
		if (Expression->IsA<UMaterialExpressionTextureSample>())
		{
			InPosition.Y += TextureSampleOffset;
		}

		ArrangedInputs.Add(Expression);
	}
}
