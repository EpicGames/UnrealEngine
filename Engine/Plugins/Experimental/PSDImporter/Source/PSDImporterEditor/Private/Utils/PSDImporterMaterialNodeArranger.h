// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

class UMaterialExpression;
struct FExpressionInput;

class FPSDImporterMaterialNodeArranger
{
public:
	static void ArrangeNodes(const FExpressionInput& InMaterialChannelInput);

private:
	static void ArrangeNodes(UMaterialExpression& InExpression, FIntPoint InPosition);
};
